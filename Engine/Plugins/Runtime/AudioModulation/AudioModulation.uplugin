{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "2.0",
	"FriendlyName": "Audio Modulation",
	"Description": "Default implementation of Audio Modulation in the Unreal Audio Engine.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioModulation",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AudioModulationEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "MetaSound",
			"Enabled": true
		},
		{
			"Name": "WaveTable",
			"Enabled": true
		}
	]
}