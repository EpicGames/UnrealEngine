[CoreRedirects]
+EnumRedirects=(OldName="FSoundModulationLFOShape",NewName="/Script/AudioModulation.ESoundModulationLFOShape")
+EnumRedirects=(OldName="ESoundModulationCurve",NewName="/Script/WaveTable.EWaveTableCurve")
+ClassRedirects=(OldName="SoundModulationWatcher",NewName="/Script/AudioModulation.AudioModulationDestination")

[/Script/AudioModulation.AudioModulationSettings]
+Parameters=/AudioModulation/BitDepth.BitDepth
+Parameters=/AudioModulation/HPFCutoffFrequency.HPFCutoffFrequency
+Parameters=/AudioModulation/LowRateFrequency.LowRateFrequency
+Parameters=/AudioModulation/LPFCutoffFrequency.LPFCutoffFrequency
+Parameters=/AudioModulation/Pan.Pan
+Parameters=/AudioModulation/Pitch.Pitch
+Parameters=/AudioModulation/SampleRate.SampleRate
+Parameters=/AudioModulation/TimeOfDay.TimeOfDay
+Parameters=/AudioModulation/Volume.Volume