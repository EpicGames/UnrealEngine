// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioModulation : ModuleRules
	{
		// Set this to false & disable MetaSound plugin dependency
		// by setting MetaSound's field '"Enabled": false' in the
		// AudioModulation.uplugin if running Modulation without
		// MetaSound support.
		public static bool bIncludeMetaSoundSupport = true;

		public AudioModulation(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioExtensions",
					"AudioMixer",
					"Core",
					"LevelSequence",
					"MovieScene",
					"MovieSceneTracks",
					"WaveTable"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioMixer",
					"AudioMixerCore",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Projects",
					"SignalProcessing",
					"TraceLog"
				}
			);

			if (bIncludeMetaSoundSupport)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"MetasoundEngine",
						"MetasoundFrontend",
						"MetasoundGraphCore"
					}
				);
			}

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"Slate",
						"SlateCore"
					}
				);
			}

			PublicDefinitions.Add("WITH_AUDIOMODULATION=1");
			if (bIncludeMetaSoundSupport)
			{
				PublicDefinitions.Add("WITH_AUDIOMODULATION_METASOUND_SUPPORT=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_AUDIOMODULATION_METASOUND_SUPPORT=0");
			}

			PrivateDefinitions.AddRange(
				new string[]
				{
					"METASOUND_PLUGIN=AudioModulation",
					"METASOUND_MODULE=AudioModulation"
				});
		}
	}
}
