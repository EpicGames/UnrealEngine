// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioModulation.h"

#include "AudioModulationLogging.h"
#include "AudioModulationSettings.h"
#include "AudioModulationSystem.h"
#include "CanvasTypes.h"
#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"
#include "IAudioModulation.h"
#include "Modules/ModuleManager.h"
#include "SoundControlBusMix.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatch.h"
#include "SoundModulatorAsset.h"
#include "UObject/NoExportTypes.h"
#include "AudioModulationStatics.h"

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "MetasoundFrontendRegistries.h"

REGISTER_METASOUND_DATATYPE(AudioModulation::FSoundModulatorAsset, "Modulator", Metasound::ELiteralType::UObjectProxy, USoundModulatorBase);
REGISTER_METASOUND_DATATYPE(AudioModulation::FSoundModulationParameterAsset, "ModulationParameter", Metasound::ELiteralType::UObjectProxy, USoundModulationParameter);
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT



namespace AudioModulationCVars
{
	static bool bAudioModulationEnabledCVar = true;
	FAutoConsoleVariableRef CVarAudioModulationEnabled(
		TEXT("au.EnableAudioModulation"),
		bAudioModulationEnabledCVar,
		TEXT("Set to 0 to disable Audio Modulation entirely.\n"),
		ECVF_Default);
}

namespace AudioModulation
{
	FAudioModulationManager::FAudioModulationManager()
		: ModSystem(new FAudioModulationSystem())
	{
	}

	FAudioModulationManager::~FAudioModulationManager()
	{
		delete ModSystem;
	}

	void FAudioModulationManager::Initialize(const FAudioPluginInitializationParams& InitializationParams)
	{
		ModSystem->Initialize(InitializationParams);
	}

	void FAudioModulationManager::OnAuditionEnd()
	{
		ModSystem->OnAuditionEnd();
	}

	void FAudioModulationManager::GetControlBusAssetPath(TArray<FTopLevelAssetPath>& OutClassNames)
	{
		OutClassNames.Add(USoundControlBus::StaticClass()->GetClassPathName());
	}

	void FAudioModulationManager::ActivateBus(const USoundControlBus& InBus)
	{
		UE_LOGF(LogAudioModulation, VeryVerbose, "FAudioModulationManager::ActivateBus is deprecated. Use UAudioModulationDestination API (see SetModulator) instead to enforce bus lifetime");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModSystem->ActivateBus(InBus);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FAudioModulationManager::ActivateBusMix(const USoundControlBusMix& InBusMix)
	{
		ModSystem->ActivateBusMix(InBusMix);
	}

	void FAudioModulationManager::ActivateGenerator(const USoundModulationGenerator& InGenerator)
	{
		UE_LOGF(LogAudioModulation, VeryVerbose, "FAudioModulationManager::ActivateGenerator is deprecated. Use UAudioModulationDestination API (see SetModulator) instead to enforce generator lifetime");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModSystem->ActivateGenerator(InGenerator);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	USoundControlBusMix* FAudioModulationManager::CreateBusMixFromValue(FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime, float ReleaseTime, UObject* InOuter)
	{
		return ModSystem->CreateBusMixFromValue(Name, Buses, Value, AttackTime, ReleaseTime, InOuter);
	}

	void FAudioModulationManager::DeactivateBus(const USoundControlBus& InBus)
	{
		UE_LOGF(LogAudioModulation, VeryVerbose, "FAudioModulationManager::DeactivateBus is deprecated. Use UAudioModulationDestination API (see ClearModulator) instead to enforce bus lifetime.");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModSystem->DeactivateBus(InBus);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FAudioModulationManager::DeactivateBusMix(const USoundControlBusMix& InBusMix)
	{
		ModSystem->DeactivateBusMix(InBusMix);
	}

	void FAudioModulationManager::DeactivateBusMixByID(const uint32 ID)
	{
		ModSystem->DeactivateBusMixByID(ID);
	}

	void FAudioModulationManager::DeactivateAllBusMixes()
	{
		ModSystem->DeactivateAllBusMixes();
	}

	void FAudioModulationManager::DeactivateGenerator(const USoundModulationGenerator& InGenerator)
	{
		UE_LOGF(LogAudioModulation, VeryVerbose, "FAudioModulationManager::DeactivateGenerator is deprecated. Use UAudioModulationDestination API (see ClearModulator) instead to enforce generator lifetime");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModSystem->DeactivateGenerator(InGenerator);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool FAudioModulationManager::IsBusMixActive(const USoundControlBusMix& InBusMix)
	{
		return ModSystem->IsControlBusMixActive(InBusMix);
	}

#if !UE_BUILD_SHIPPING
	void FAudioModulationManager::SetDebugBusFilter(const FString* InNameFilter)
	{
		ModSystem->SetDebugBusFilter(InNameFilter);
	}

	void FAudioModulationManager::SetDebugGeneratorFilter(const FString* InFilter)
	{
		ModSystem->SetDebugGeneratorFilter(InFilter);
	}

	void FAudioModulationManager::SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled)
	{
		ModSystem->SetDebugGeneratorTypeFilter(InFilter, bInIsEnabled);
	}

	void FAudioModulationManager::SetDebugActiveMixesEnabled(bool bInIsEnabled)
	{
		ModSystem->SetDebugActiveMixesEnabled(bInIsEnabled);
	}

	void FAudioModulationManager::SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled)
	{
		ModSystem->SetDebugActiveGlobalMixesEnabled(bInIsEnabled);
	}

	void FAudioModulationManager::SetDebugGeneratorsEnabled(bool bInIsEnabled)
	{
		ModSystem->SetDebugGeneratorsEnabled(bInIsEnabled);
	}

	void FAudioModulationManager::SetDebugMatrixEnabled(bool bInIsEnabled)
	{
		ModSystem->SetDebugMatrixEnabled(bInIsEnabled);
	}

	void FAudioModulationManager::SetDebugMixFilter(const FString* InNameFilter)
	{
		ModSystem->SetDebugMixFilter(InNameFilter);
	}

#endif // !UE_BUILD_SHIPPING

	void FAudioModulationManager::SaveMixToProfile(const USoundControlBusMix& InBusMix, const int32 InProfileIndex)
	{
		ModSystem->SaveMixToProfile(InBusMix, InProfileIndex);
	}

	TArray<FSoundControlBusMixStage> FAudioModulationManager::LoadMixFromProfile(const int32 InProfileIndex, USoundControlBusMix& OutBusMix)
	{
		return ModSystem->LoadMixFromProfile(InProfileIndex, OutBusMix);
	}

	void FAudioModulationManager::UpdateMix(const TArray<FSoundControlBusMixStage>& InStages, USoundControlBusMix& InOutMix, bool bInUpdateObject, float InFadeTime, double Duration, bool bRetriggerOnActivation)
	{
		ModSystem->UpdateMix(InStages, InOutMix, bInUpdateObject, InFadeTime, Duration, bRetriggerOnActivation);
	}

	void FAudioModulationManager::UpdateMix(const USoundControlBusMix& InMix, float InFadeTime)
	{
		ModSystem->UpdateMix(InMix, InFadeTime);
	}

	void FAudioModulationManager::UpdateMixByFilter(const FString& InAddressFilter, const TSubclassOf<USoundModulationParameter>& InParamClassFilter, USoundModulationParameter* InParamFilter, float Value, float FadeTime, USoundControlBusMix& InOutMix, bool bInUpdateObject)
	{
		ModSystem->UpdateMixByFilter(InAddressFilter, InParamClassFilter, InParamFilter, Value, FadeTime, InOutMix, bInUpdateObject);
	}

	void FAudioModulationManager::SoloBusMix(const USoundControlBusMix& InBusMix)
	{
		ModSystem->SoloBusMix(InBusMix);
	}

	void FAudioModulationManager::SetGlobalBusMixValue(USoundControlBus& InBus, float InValue, float InFadeTime)
	{
		ModSystem->SetGlobalBusMixValue(InBus, InValue, InFadeTime);
	}

	void FAudioModulationManager::ClearGlobalBusMixValue(const USoundControlBus& InBus, float InFadeTime)
	{
		ModSystem->ClearGlobalBusMixValue(InBus, InFadeTime);
	}

	void FAudioModulationManager::ClearAllGlobalBusMixValues(float InFadeTime)
	{
		ModSystem->ClearAllGlobalBusMixValues(InFadeTime);
	}

	float FAudioModulationManager::GetModulatorValueThreadSafe(uint32 ModulationID)
	{
		float OutValue = 1.0f;
		ModSystem->GetModulatorValueThreadSafe(ModulationID, OutValue);
		return OutValue;
	}

#if !UE_BUILD_SHIPPING
	bool FAudioModulationManager::OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		return ModSystem->OnPostHelp(ViewportClient, Stream);
	}

	int32 FAudioModulationManager::OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		return ModSystem->OnRenderStat(Viewport, Canvas, X, Y, Font, ViewLocation, ViewRotation);
	}

	bool FAudioModulationManager::OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		return ModSystem->OnToggleStat(ViewportClient, Stream);
	}
#endif // !UE_BUILD_SHIPPING

	void FAudioModulationManager::ProcessModulators(const double InElapsed)
	{
		if (!AudioModulationCVars::bAudioModulationEnabledCVar)
		{
			return;
		}
		ModSystem->ProcessModulators(InElapsed);
	}

	void FAudioModulationManager::RegisterModulator(Audio::FModulatorHandleId InHandleId, Audio::FModulatorId InModulatorId)
	{
		ModSystem->RegisterModulator(InHandleId, InModulatorId);
	}

	bool FAudioModulationManager::GetModulatorValue(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const
	{
		return ModSystem->GetModulatorValue(ModulatorHandle, OutValue);
	}

	bool FAudioModulationManager::GetModulatorValueThreadSafe(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const
	{
		return ModSystem->GetModulatorValueThreadSafe(ModulatorHandle, OutValue);
	}

	FAudioModulationSystem& FAudioModulationManager::GetSystem()
	{
		check(ModSystem);
		return *ModSystem;
	}

	void FAudioModulationManager::UnregisterModulator(const Audio::FModulatorHandle& InHandle)
	{
		ModSystem->UnregisterModulator(InHandle);
	}

	void FAudioModulationManager::UpdateModulator(const USoundModulatorBase& InModulator)
	{
		ModSystem->UpdateModulator(InModulator);
	}

	FAudioModulationManager* GetDeviceModulationManager(Audio::FDeviceId InDeviceId)
	{
		FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get();
		if (ensure(DeviceManager))
		{
			FAudioDevice* AudioDevice = DeviceManager->GetAudioDeviceRaw(InDeviceId);
			if (ensure(AudioDevice))
			{
				if (AudioDevice->IsModulationPluginEnabled())
				{
					if (IAudioModulationManager* Modulation = AudioDevice->ModulationInterface.Get())
					{
						return static_cast<FAudioModulationManager*>(Modulation);
					}
				}
			}
		}

		return nullptr;
	}

	void IterateModulationManagers(TFunctionRef<void(FAudioModulationManager&)> InFunction)
	{
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			TArray<FAudioDevice*> Devices = DeviceManager->GetAudioDevices();
			DeviceManager->IterateOverAllDevices([ModFunction = MoveTemp(InFunction)](Audio::FDeviceId DeviceId, FAudioDevice* AudioDevice)
			{
				if (AudioDevice && AudioDevice->IsModulationPluginEnabled() && AudioDevice->ModulationInterface.IsValid())
				{
					auto ModulationInterface = static_cast<AudioModulation::FAudioModulationManager*>(AudioDevice->ModulationInterface.Get());
					ModFunction(*ModulationInterface);
				}
			});
		}
	}
} // namespace AudioModulation

TAudioModulationPtr FAudioModulationPluginFactory::CreateNewModulationPlugin(FAudioDevice* OwningDevice)
{
	return TAudioModulationPtr(new AudioModulation::FAudioModulationManager());
}

void FAudioModulationModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::AudioMixerPlugins);
	IModularFeatures::Get().RegisterModularFeature(FAudioModulationPluginFactory::GetModularFeatureName(), &ModulationPluginFactory);

	if (const UAudioModulationSettings* ModulationSettings = GetDefault<UAudioModulationSettings>())
	{
		ModulationSettings->RegisterParameters();
	}

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
	UE_LOGF(LogAudioModulation, Log, "Registering Modulation MetaSound Nodes...");

	// All MetaSound interfaces are required to be loaded prior to registering & loading MetaSound assets,
	// so check that the MetaSoundEngine is loaded prior to pending Modulation defined classes
	FModuleManager::Get().LoadModuleChecked("MetasoundEngine");

	METASOUND_REGISTER_ITEMS_IN_MODULE
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT

	UE_LOGF(LogAudioModulation, Log, "Audio Modulation Initialized");
}

void FAudioModulationModule::ShutdownModule()
{
	LLM_SCOPE(ELLMTag::AudioMixerPlugins);
	IModularFeatures::Get().UnregisterModularFeature(FAudioModulationPluginFactory::GetModularFeatureName(), &ModulationPluginFactory);

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
	UE_LOGF(LogAudioModulation, Log, "Unregistering Modulation MetaSound Nodes...");
	METASOUND_UNREGISTER_ITEMS_IN_MODULE
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT

	UE_LOGF(LogAudioModulation, Log, "Audio Modulation Shutdown");
}

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
#endif
IMPLEMENT_MODULE(FAudioModulationModule, AudioModulation);
