// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationDebugger.h"
#include "ViewportClient.h"

#if WITH_AUDIOMODULATION
#if !UE_BUILD_SHIPPING
#include "CoreMinimal.h"
#include "Async/TaskGraphInterfaces.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationSystem.h"
#include "AudioThread.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "IAudioModulation.h"
#include "Features/IModularFeatures.h"
#include "Misc/CoreDelegates.h"
#include "SoundControlBusProxy.h"
#include "SoundModulationPatch.h"
#include "SoundModulationProxy.h"
#include "SoundModulationValue.h"

namespace AudioModulation
{
	namespace Debug
	{
		using FDebugViewUpdateFunction = TUniqueFunction<void(FAudioModulationManager& /*InModulation*/, const FString* /*InFilter*/)>;
		void UpdateObjectFilter(const TArray<FString>& Args, UWorld* World, FDebugViewUpdateFunction InUpdateViewFunc)
		{
			if (!World)
			{
				return;
			}

			const FString* FilterString = nullptr;
			if (Args.Num() > 0)
			{
				FilterString = &Args[0];
			}

			FAudioDeviceHandle DeviceHandle = World->GetAudioDevice();
			if (DeviceHandle.IsValid())
			{
				if (GEngine)
				{
					GEngine->Exec(World, TEXT("au.Debug.Modulation 1"));
				}

				if (IAudioModulationManager* Modulation = DeviceHandle->ModulationInterface.Get())
				{
					FAudioModulationManager& ModulationImpl = *static_cast<FAudioModulationManager*>(Modulation);
					InUpdateViewFunc(ModulationImpl, FilterString);
				}
			}
		}
	}
}

static float AudioModulationDebugUpdateRateCVar = 0.1f;
FAutoConsoleVariableRef CVarAudioModulationDebugUpdateRate(
	TEXT("au.Debug.Modulation.UpdateRate"),
	AudioModulationDebugUpdateRateCVar,
	TEXT("Sets update rate for modulation debug statistics (in seconds).\n")
	TEXT("Default: 0.1f"),
	ECVF_Default);

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugBuses(
	TEXT("au.Debug.Modulation.Filter.Buses"),
	TEXT("Sets substring by which to filter mixes in matrix view. Arguments:\n"
		"Enabled (Optional, ex. True, False. Default: True) - Whether or not to enable showing buses.\n"
		"Filter (Optional, Default: null) - Whether or not to filter buses by name using the provided substring.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			InModulation.SetDebugBusFilter(InFilter);
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugMixes(
	TEXT("au.Debug.Modulation.Filter.Mixes"),
	TEXT("Sets substring by which to filter mixes in matrix view. Arguments:\n"
		"Filter - Filter bus mixes by name using the provided substring.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			InModulation.SetDebugMixFilter(InFilter);
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugMatrix(
	TEXT("au.Debug.Modulation.Enable.Matrix"),
	TEXT("Whether or not to enable mix matrix. Arguments:\n"
		"Enabled (Default: 1/True) - Whether or not to have matrix view enabled.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			if (InFilter)
			{
				const bool bEnabled = InFilter->ToBool();
				InModulation.SetDebugMatrixEnabled(bEnabled);
			}
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationSetDebugGeneratorFilter(
	TEXT("au.Debug.Modulation.Filter.Generators"),
	TEXT("Sets substring by which to filter generators. Arguments:\n"
		"Name - Filter generators by name using the provided substring.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;

		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			InModulation.SetDebugGeneratorFilter(InFilter);
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationSetDebugGeneratorFilterType(
	TEXT("au.Debug.Modulation.Filter.Generators.Type"),
	TEXT("Whether to display or hide Generator type provided (defaults to show if enablement boolean not provided). Arguments:\n"
		"Name (Optional, Default: null) - Filter generators type to display/hide (Empty/null clears any currently set filter).\n"
		"Enabled (Optional, Default: True) - True to show, false to hide.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;

		bool bEnabled = true;
		if (Args.Num() > 1)
		{
			bEnabled = Args[1].ToBool();
		}
		Debug::UpdateObjectFilter(Args, World, [bEnabled](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			InModulation.SetDebugGeneratorTypeFilter(InFilter, bEnabled);
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugEnableGenerators(
	TEXT("au.Debug.Modulation.Enable.Generators"),
	TEXT("Whether or not to enable displaying generators. Arguments:\n"
		"Enabled (Default: 1/True) - Whether or not to have generators view enabled.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			if (InFilter)
			{
				const bool bEnabled = InFilter->ToBool();
				InModulation.SetDebugGeneratorsEnabled(bEnabled);
			}
		});
	}));


static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugActiveMixes(
	TEXT("au.Debug.Modulation.ActiveMixes"),
	TEXT("Whether or not to enable displaying active Control Bus Mixes. Arguments:\n"
		"Enabled (Default: 1/True) - Whether or not to have generators view enabled.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			if (InFilter)
			{
				const bool bEnabled = InFilter->ToBool();
				InModulation.SetDebugActiveMixesEnabled(bEnabled);
			}
		});
	}));

static FAutoConsoleCommandWithWorldAndArgs CVarAudioModulationDebugShowGlobalMixes(
	TEXT("au.Debug.Modulation.ActiveMixes.ShowGlobalMixes"),
	TEXT("Whether or not to show Global Mixes when displaying active mixes using 'au.Debug.Modulation.ActiveMixes'. Arguments:\n"
		"Enabled (Default: 1/True) - Whether or not to have generators view enabled.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		using namespace AudioModulation;
		Debug::UpdateObjectFilter(Args, World, [](FAudioModulationManager& InModulation, const FString* InFilter)
		{
			if (InFilter)
			{
				const bool bEnabled = InFilter->ToBool();
				InModulation.SetDebugActiveGlobalMixesEnabled(bEnabled);
			}
		});
	}));

namespace AudioModulation
{
	namespace Debug
	{
		const int32 MaxNameLength = 40;
		const int32 XIndent = 36;

		FColor GetUnitRenderColor(const float Value)
		{
			return Value > 1.0f || Value < 0.0f
				? FColor::Red
				: FColor::Green;
		}

		int32 RenderStatGenerators(const FString& Name, const FGeneratorDebugInfo& GeneratorDebugInfo, int32 CellWidth, FCanvas& Canvas, int32 X, int32 Y, const UFont& Font)
		{
			int32 Height = 12;
			int32 Width = 12;
			Font.GetStringHeightAndWidth(TEXT("@"), Height, Width);

			Y += Height;
			X += XIndent;

			Canvas.DrawShadowedString(X, Y, *(Name + TEXT(":")), &Font, FColor::Red);
			Y += Height;

			// Print Category Titles
			{
				int32 RowX = X;
				for (const FString& Category : GeneratorDebugInfo.Categories)
				{
					Canvas.DrawShadowedString(RowX, Y, *Category, &Font, FColor::White);
					RowX += Width * CellWidth;
				}
			}

			// Print Instance Info
			for (const FGeneratorDebugInfo::FInstanceValues& InstanceValues : GeneratorDebugInfo.FilteredInstances)
			{
				Y += Height;

				int32 RowX = X;

				// Validate that all instances have the same number of values as there are registered categories.
				ensure(InstanceValues.Num() == GeneratorDebugInfo.Categories.Num());

				for (const FString& Value : InstanceValues)
				{
					FString DisplayValue = Value.Left(MaxNameLength);
					if (DisplayValue.Len() < MaxNameLength)
					{
						DisplayValue = DisplayValue.RightPad(MaxNameLength - DisplayValue.Len());
					}

					Canvas.DrawShadowedString(RowX, Y, *DisplayValue, &Font, FColor::Green);
					RowX += Width * CellWidth;
				}
			}

			// Add space after, so twice the height
			Y += Height * 2;

			return Y;
		}

		int32 RenderStatMixMatrix(const TArray<FControlBusMixDebugInfo>& FilteredMixes, const TArray<FControlBusDebugInfo>& FilteredBuses, FCanvas& Canvas, int32 X, int32 Y, const UFont& Font)
		{
			int32 Height = 12;
			int32 Width = 12;
			Font.GetStringHeightAndWidth(TEXT("@"), Height, Width);
			X += XIndent;

			Canvas.DrawShadowedString(X, Y, TEXT("Bus Mix Matrix:"), &Font, FColor::Red);
			Y += Height;

			// Determine minimum width of cells
			static const FString MixSubTotalHeader = TEXT("Mix");
			static const FString GeneratorsSubTotalHeader = TEXT("Generators");
			static const FString TotalHeader = TEXT("Final");
			static const TArray<int32> StaticCellWidths =
			{
				GeneratorsSubTotalHeader.Len(),
				MixSubTotalHeader.Len(),
				TotalHeader.Len(),
				14 /* Minimum width due to value strings X.XXXX(X.XXXX) */
			};

			int32 CellWidth = FMath::Max(StaticCellWidths);
			for (const FControlBusMixDebugInfo& BusMix : FilteredMixes)
			{
				CellWidth = FMath::Max(CellWidth, BusMix.Name.Len());
			}
			for (const FControlBusDebugInfo& Bus : FilteredBuses)
			{
				CellWidth = FMath::Max(CellWidth, Bus.Name.Len());
			}

			// Draw Column Headers
			int32 RowX = X;
			for (const FControlBusDebugInfo& Bus : FilteredBuses)
			{
				FString Name = Bus.Name.Left(MaxNameLength);
				Name += FString::Printf(TEXT(" (%u)"), Bus.RefCount);

				if (Name.Len() < MaxNameLength)
				{
					Name = Name.RightPad(MaxNameLength - Name.Len());
				}

				RowX += Width * CellWidth; // Add before to leave space for row headers
				Canvas.DrawShadowedString(RowX, Y, *Name, &Font, FColor::White);
			}

			// Draw Row Headers
			int32 ColumnY = Y;
			for (const FControlBusMixDebugInfo& BusMix : FilteredMixes)
			{
				FString Name = BusMix.Name.Left(MaxNameLength);
				Name += FString::Printf(TEXT(" (%u)"), BusMix.RefCount);

				if (Name.Len() < MaxNameLength)
				{
					Name = Name.RightPad(MaxNameLength - Name.Len());
				}

				ColumnY += Height; // Add before to leave space for column headers
				Canvas.DrawShadowedString(X, ColumnY, *Name, &Font, FColor::White);
			}

			// Reset Corner of Matrix & Draw Per Bus Data
			RowX = X;
			ColumnY = Y;
			for (const FControlBusMixDebugInfo& BusMix : FilteredMixes)
			{
				ColumnY += Height; // Add before to leave space for column headers
				RowX = X;

				for (const FControlBusDebugInfo& Bus : FilteredBuses)
				{
					RowX += Width * CellWidth; // Add before to leave space for row headers

					float Target = Bus.DefaultValue;
					float Value = Bus.DefaultValue;
					if (const FControlBusMixStageDebugInfo* Stage = BusMix.Stages.Find(Bus.Id))
					{
						Target = Stage->TargetValue;
						Value  = Stage->CurrentValue;
					}

					if (Target != Value)
					{
						Canvas.DrawShadowedString(RowX, ColumnY, *FString::Printf(TEXT("%.4f(%.4f)"), Value, Target), &Font, FColor::Green);
					}
					else
					{
						Canvas.DrawShadowedString(RowX, ColumnY, *FString::Printf(TEXT("%.4f"), Value), &Font, FColor::Green);
					}
				}
			}

			Y += (FilteredMixes.Num() + 2) * Height; // Add 2, one for header and one for spacing row

			// Draw Sub-Totals & Totals
			Canvas.DrawShadowedString(X, Y, *MixSubTotalHeader, &Font, FColor::Yellow);
			RowX = X;
			for (const FControlBusDebugInfo& Bus : FilteredBuses)
			{
				RowX += Width * CellWidth; // Add before to leave space for row headers

				const float Value = Bus.MixValue;
				if (FMath::IsNaN(Value))
				{
					Canvas.DrawShadowedString(RowX, Y, TEXT("N/A"), &Font, FColor::Green);
				}
				else
				{
					const FColor Color = Debug::GetUnitRenderColor(Value);
					Canvas.DrawShadowedString(RowX, Y, *FString::Printf(TEXT("%.4f"), Value), &Font, Color);
				}
			}
			Y += Height;

			Canvas.DrawShadowedString(X, Y, *GeneratorsSubTotalHeader, &Font, FColor::Yellow);
			RowX = X;
			for (const FControlBusDebugInfo& Bus : FilteredBuses)
			{
				RowX += Width * CellWidth; // Add before to leave space for row headers
				const float Value = Bus.GeneratorValue;
				const FColor Color = Debug::GetUnitRenderColor(Value);
				Canvas.DrawShadowedString(RowX, Y, *FString::Printf(TEXT("%.4f"), Value), &Font, Color);
			}

			Y += Height;
			Canvas.DrawShadowedString(X, Y, *TotalHeader, &Font, FColor::Yellow);
			RowX = X;
			for (const FControlBusDebugInfo& Bus : FilteredBuses)
			{
				RowX += Width * CellWidth; // Add before to leave space for row headers
				const FColor Color = Debug::GetUnitRenderColor(Bus.Value);
				Canvas.DrawShadowedString(RowX, Y, *FString::Printf(TEXT("%.4f"), Bus.Value), &Font, Color);
			}
			Y += Height;

			return Y + Height;
		}

		int32 RenderStatActiveMixes(const TArray<FControlBusMixDebugInfo>& FilteredMixes, bool bShowGlobalMixes, FCanvas& Canvas, int32 X, int32 Y, const UFont& Font)
		{
			FAudioDevice* AudioDevice = nullptr;

			int32 Height = 12;
			int32 Width = 12;
			Font.GetStringHeightAndWidth(TEXT("@"), Height, Width);
			Canvas.DrawShadowedString(X, Y, TEXT("Active Control Bus Mixes:"), &Font, FColor::Green);
			Y += Height;
			X += XIndent;
			
			bool bDisplayedBusMixes = false;

			if (FilteredMixes.Num() > 0)
			{
			 	bDisplayedBusMixes = true;

				for (const FControlBusMixDebugInfo& DebugMix : FilteredMixes)
				{
					if (!bShowGlobalMixes && DebugMix.bIsGlobal)
					{
						continue;
					}
					
					const FString BusMixInfo = FString::Printf(TEXT("%s - Timer: %.3f - Total Ref Count: %i"), *DebugMix.Name, DebugMix.Timer, DebugMix.RefCount);
					const FColor& TextColor = (FColor::White);
					Canvas.DrawShadowedString(X, Y, *BusMixInfo, &Font, TextColor);
					Y += Height;
				}
			}

			if (!bDisplayedBusMixes)
			{
				Canvas.DrawShadowedString(X, Y, TEXT("None"), &Font, FColor::White);
				Y += Height;
			}
			
			Y += Height;
			return Y;
		}
		
		template <typename T>
		bool CompareNames(const T& A, const T& B)
		{
			return A.Name < B.Name;
		}

		template <typename T>
		void FilterDebugInfoArray(const TArray<T>& InArray, const FString& FilterString, int32 MaxCount, TArray<T>& FilteredArray)
		{
			int32 FilteredItemCount = 0;
			for (const T& Item : InArray)
			{
				const bool Filtered = !FilterString.IsEmpty() && !Item.Name.Contains(FilterString);
				if (!Filtered)
				{
					FilteredArray.Add(Item);
					if (++FilteredItemCount >= MaxCount)
					{
						return;
					}
				}
			}
		}

		template <typename T, typename U>
		void FilterDebugArray(const T& Map, const FString& FilterString, int32 MaxCount, TArray<const U*>& FilteredArray)
		{
			int32 FilteredItemCount = 0;
			for (const auto& IdItemPair : Map)
			{
				const U& Item = IdItemPair.Value;
				const bool Filtered = !FilterString.IsEmpty()
					&& !Item.GetName().ToString().Contains(FilterString);
				if (!Filtered)
				{
					FilteredArray.Add(&Item);
					if (++FilteredItemCount >= MaxCount)
					{
						return;
					}
				}
			}
		}

		template <typename T, typename U, typename K>
		void FilterDebugMap(const T& Map, const FString& FilterString, int32 MaxCount, TMap<K, const U*>& OutFilteredMap)
		{
			int32 FilteredItemCount = 0;
			for (const TPair<K, U>& IdItemPair : Map)
			{
				const U& Item = IdItemPair.Value;
				const bool Filtered = !FilterString.IsEmpty()
					&& !Item.GetName().ToString().Contains(FilterString);
				if (!Filtered)
				{
					OutFilteredMap.Add(IdItemPair.Key, &Item);
					if (++FilteredItemCount >= MaxCount)
					{
						return;
					}
				}
			}
		}
	} // namespace Debug

	FAudioModulationDebugger::FAudioModulationDebugger()
		: bActive(0)
		, bShowRenderStatMix(1)
		, bShowGenerators(0)
		, bEnableAllGenerators(0)
		, bShowActiveMixes(0)
		, bShowGlobalMixes(1)
		, ElapsedSinceLastUpdate(0.0f)
	{
		IModularFeatures::FScopedLockModularFeatureList ModularFeatureListLock;
		IModularFeatures::Get().RegisterModularFeature(DebugDataModularFeatureName, this);
	}

	FAudioModulationDebugger::~FAudioModulationDebugger()
	{
		IModularFeatures::FScopedLockModularFeatureList ModularFeatureListLock;
		IModularFeatures::Get().UnregisterModularFeature(DebugDataModularFeatureName, this);
	}

	void FAudioModulationDebugger::UpdateDebugData(double InElapsed, const FReferencedProxies& InRefProxies, Audio::FDeviceId DeviceId)
	{
		if (!bActive)
		{
			ElapsedSinceLastUpdate = 0.0f;
			return;
		}

		ElapsedSinceLastUpdate += InElapsed;
		if (AudioModulationDebugUpdateRateCVar > ElapsedSinceLastUpdate)
		{
			return;
		}
		ElapsedSinceLastUpdate = 0.0f;

		//Create Bus Debug Info Arrays (filtered and unfiltered)
		TArray<FControlBusDebugInfo> RefreshedUnfilteredBuses;
		for (const TTuple<FBusId, FControlBusProxy>& BusProxyMap : InRefProxies.Buses)
		{
			const FControlBusProxy& Proxy = BusProxyMap.Value;
			FControlBusDebugInfo DebugInfo;
			DebugInfo.DefaultValue = Proxy.GetDefaultValue();
			DebugInfo.Id = Proxy.GetId();
			DebugInfo.GeneratorValue = Proxy.GetGeneratorValue();
			DebugInfo.MixValue = Proxy.GetMixValue();
			DebugInfo.Name = Proxy.GetName().ToString();
			DebugInfo.ParameterName = Proxy.GetParameterName().ToString();
			DebugInfo.RefCount = Proxy.GetRefCount();
			DebugInfo.Value = Proxy.GetValue();
			RefreshedUnfilteredBuses.Add(DebugInfo);
		}

		RefreshedUnfilteredBuses.Sort(&Debug::CompareNames<FControlBusDebugInfo>);

		TArray<FControlBusDebugInfo> RefreshedFilteredBuses;
		static const int32 MaxFilteredBuses = 8;
		Debug::FilterDebugInfoArray<FControlBusDebugInfo>(RefreshedUnfilteredBuses, BusStringFilter, MaxFilteredBuses,RefreshedFilteredBuses);

		
		FAudioModulationSystem* ModulationSystem = nullptr;
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDeviceHandle DeviceHandle = DeviceManager->GetAudioDevice(DeviceId))
			{
				if (IAudioModulationManager* Modulation = DeviceHandle->ModulationInterface.Get())
				{
					if (FAudioModulationManager* ModulationManager = static_cast<FAudioModulationManager*>(Modulation))
					{
						ModulationSystem = &ModulationManager->GetSystem();
					}
				}
			}
		}

		//Create Bus Mix Debug Info Arrays (filtered and unfiltered)
		TArray<FControlBusMixDebugInfo> RefreshedUnfilteredMixes;
		TArray<FControlBusMixDebugInfo> RefreshedFilteredMixes;

		if (ModulationSystem)
		{
			TMap<FBusMixId, const FModulatorBusMixProxy*> UnfilteredMixProxies;
			for (const TTuple<FBusMixId, FModulatorBusMixProxy>& Pair : InRefProxies.BusMixes)
			{
				const FModulatorBusMixProxy& Proxy = Pair.Value;
			
				FControlBusMixDebugInfo DebugInfo;
				DebugInfo.Name = Proxy.GetName().ToString();
				DebugInfo.RefCount = Proxy.GetRefCount();
				DebugInfo.Id = Pair.Key;
				DebugInfo.Timer = Proxy.GetTimeRemaining();
				// Bus mix is global if it only has one stage and its ID is the Control Bus's ID.
				if (Proxy.Stages.Num() == 1)
				{
					uint32 BusId = Proxy.Stages.FindArbitraryElement()->Key;
					DebugInfo.bIsGlobal = ModulationSystem->IsBusMixGlobal(BusId);
				}
			
				for (const TPair<FBusId, FModulatorBusMixStageProxy>& Stage : Proxy.Stages)
				{
					FControlBusMixStageDebugInfo StageDebugInfo;
					StageDebugInfo.CurrentValue = Stage.Value.Value.GetCurrentValue();
					StageDebugInfo.TargetValue = Stage.Value.Value.TargetValue;
					DebugInfo.Stages.Add(Stage.Key, StageDebugInfo);
				}
				RefreshedUnfilteredMixes.Add(DebugInfo);
			}

			RefreshedUnfilteredMixes.Sort(&Debug::CompareNames<FControlBusMixDebugInfo>);
			static const int32 MaxFilteredMixes = 16;
			Debug::FilterDebugInfoArray<FControlBusMixDebugInfo>(RefreshedUnfilteredMixes, MixStringFilter, MaxFilteredMixes,RefreshedFilteredMixes);
			
		}
		
		static const int32 MaxFilteredGenerators = 8;
		TArray<const FModulatorGeneratorProxy*> FilteredGeneratorProxies;
		Debug::FilterDebugArray<FGeneratorProxyMap, FModulatorGeneratorProxy>(InRefProxies.Generators, GeneratorStringFilter, MaxFilteredGenerators, FilteredGeneratorProxies);

		FGeneratorSortMap RefreshedFilteredGenerators;
		for (const FModulatorGeneratorProxy* Proxy : FilteredGeneratorProxies)
		{
			FGeneratorDebugInfo::FInstanceValues Values = Proxy->GetDebugValues();
			FGeneratorDebugInfo& DebugInfo = RefreshedFilteredGenerators.FindOrAdd(Proxy->GetDebugName());
			DebugInfo.FilteredInstances.Emplace(MoveTemp(Values));
			DebugInfo.Categories = Proxy->GetDebugCategories();
		}

		FFunctionGraphTask::CreateAndDispatchWhenReady([DebuggerPtr = AsWeak(), RefreshedUnfilteredBuses, RefreshedUnfilteredMixes, RefreshedFilteredBuses, RefreshedFilteredMixes, RefreshedFilteredGenerators, DeviceId]()
		{
			TSharedPtr<FAudioModulationDebugger> ThisDebugger = DebuggerPtr.Pin();
			if (!ThisDebugger.IsValid())
			{
				return;
			}

			ThisDebugger->AssociatedDeviceId = DeviceId;	

			ThisDebugger->FilteredBuses = RefreshedFilteredBuses;
			ThisDebugger->UnfilteredBuses = RefreshedUnfilteredBuses;

			ThisDebugger->FilteredMixes = RefreshedFilteredMixes;
			ThisDebugger->UnfilteredMixes = RefreshedUnfilteredMixes;

			FGeneratorSortMap NewFilteredMap;
			if (ThisDebugger->bShowGenerators)
			{
				NewFilteredMap = RefreshedFilteredGenerators;
				for (TPair<FString, FGeneratorDebugInfo>& NewInfoPair : NewFilteredMap)
				{
					FGeneratorDebugInfo* LastDebugInfo = ThisDebugger->FilteredGeneratorsMap.Find(NewInfoPair.Key);
					NewInfoPair.Value.bEnabled = ThisDebugger->bEnableAllGenerators
						|| (LastDebugInfo && LastDebugInfo->bEnabled);

					if (bool* Value = ThisDebugger->RequestedGeneratorUpdate.Find(NewInfoPair.Key))
					{
						NewInfoPair.Value.bEnabled = *Value;
					}

					if (!NewInfoPair.Value.bEnabled)
					{
						NewInfoPair.Value.FilteredInstances.Reset();
					}
				}
				ThisDebugger->bEnableAllGenerators = 0;
				ThisDebugger->RequestedGeneratorUpdate.Reset();
			}

			ThisDebugger->FilteredGeneratorsMap = MoveTemp(NewFilteredMap);
		}, TStatId(), nullptr, ENamedThreads::GameThread);
	}

	bool FAudioModulationDebugger::OnPostHelp(FCommonViewportClient& ViewportClient, const TCHAR* Stream)
	{
		if (GEngine)
		{
			static const float TimeShowHelp = 10.0f;
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("-MixFilter: Substring that filters mixes shown in matrix view"));
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("-BusFilter: Substring that filters buses shown in matrix view"));
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("-Matrix: Show bus matrix"));
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("-GeneratorFilter: Substring that filters Generator types listed"));
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("-Generator: Show Generator debug data"));
			GEngine->AddOnScreenDebugMessage(-1, TimeShowHelp, FColor::Yellow, TEXT("stat SoundModulators:"));
		}

		return true;
	}

	void FAudioModulationDebugger::SetDebugBusFilter(const FString* InNameFilter)
	{
		check(IsInGameThread());

		if (InNameFilter)
		{
			BusStringFilter = *InNameFilter;
		}
		else
		{
			BusStringFilter.Reset();
		}
	}

	void FAudioModulationDebugger::SetDebugMatrixEnabled(bool bInIsEnabled)
	{
		bShowRenderStatMix = bInIsEnabled;
	}

	void FAudioModulationDebugger::SetDebugMixFilter(const FString* InNameFilter)
	{
		check(IsInGameThread());

		if (InNameFilter)
		{
			MixStringFilter = *InNameFilter;
		}
		else
		{
			MixStringFilter.Reset();
		}
	}

	void FAudioModulationDebugger::SetDebugGeneratorsEnabled(bool bInIsEnabled)
	{
		check(IsInGameThread());

		bShowGenerators = bInIsEnabled;
		bEnableAllGenerators = 1;
	}

	void FAudioModulationDebugger::SetDebugGeneratorFilter(const FString* InFilter)
	{
		check(IsInGameThread());

		if (InFilter)
		{
			bShowGenerators = 1;
			GeneratorStringFilter = *InFilter;
		}
		else
		{
			GeneratorStringFilter.Reset();
		}
	}

	void FAudioModulationDebugger::SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled)
	{
		check(IsInGameThread());

		if (InFilter)
		{
			bShowGenerators |= bInIsEnabled;
			RequestedGeneratorUpdate.Add(*InFilter, bInIsEnabled);
		}
		else
		{
			bShowGenerators = bInIsEnabled;
			if (bInIsEnabled)
			{
				bEnableAllGenerators = 1;
			}
		}
	}

	void FAudioModulationDebugger::SetDebugActiveMixesEnabled(bool bInIsEnabled)
	{
		bShowActiveMixes = bInIsEnabled;
	}

	void FAudioModulationDebugger::SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled)
	{
		bShowGlobalMixes = bInIsEnabled;
		bShowActiveMixes = 1;
	}

	int32 FAudioModulationDebugger::OnRenderStat(FCanvas& Canvas, int32 X, int32 Y, const UFont& Font)
	{
		check(IsInGameThread());

		// Render stats can get called when toggle did not update bActive, so force active
		// if called and update toggle state accordingly, utilizing the last set filters.
		if (!bActive)
		{
			bActive = true;
		}

		if (bShowRenderStatMix)
		{
			Y = Debug::RenderStatMixMatrix(FilteredMixes, FilteredBuses, Canvas, X, Y, Font);
		}

		int32 CellWidth = 0;
		for (const TPair<FString, FGeneratorDebugInfo>& GeneratorStatPair : FilteredGeneratorsMap)
		{
			for (const FString& Category : GeneratorStatPair.Value.Categories)
			{
				CellWidth = FMath::Max(CellWidth, Category.Len());
			}

			for (const FGeneratorDebugInfo::FInstanceValues& InstanceInfo : GeneratorStatPair.Value.FilteredInstances)
			{
				for (const FString& Value : InstanceInfo)
				{
					CellWidth = FMath::Max(CellWidth, Value.Len());
				}
			}
		}

		if (bShowGenerators)
		{
			for (const TPair<FString, FGeneratorDebugInfo>& GeneratorStatPair : FilteredGeneratorsMap)
			{
				if (GeneratorStatPair.Value.bEnabled)
				{
					Y = Debug::RenderStatGenerators(GeneratorStatPair.Key, GeneratorStatPair.Value, CellWidth, Canvas, X, Y, Font);
				}
			}
		}

		if (bShowActiveMixes)
		{
			Y = Debug::RenderStatActiveMixes(FilteredMixes, bShowGlobalMixes, Canvas, X, Y, Font);
		}

		return Y;
	}

	bool FAudioModulationDebugger::OnToggleStat(FCommonViewportClient& ViewportClient, const TCHAR* Stream)
	{
		check(IsInGameThread());

		if (bActive == ViewportClient.IsStatEnabled(TEXT("SoundModulators")))
		{
			return true;
		}

		if (!bActive)
		{
			if (Stream && Stream[0] != '\0')
			{
				bool bGeneratorsUpdated = false;
				for (TPair<FString, FGeneratorDebugInfo>& GeneratorStatPair : FilteredGeneratorsMap)
				{
					const bool bGeneratorEnabled = FParse::Param(Stream, *GeneratorStatPair.Key);
					if (bGeneratorEnabled)
					{
						bGeneratorsUpdated = true;
						GeneratorStatPair.Value.bEnabled = bGeneratorEnabled;
					}
					else
					{
						GeneratorStatPair.Value.bEnabled = false;
					}
				}

				const bool bUpdateMix = FParse::Param(Stream, TEXT("Matrix"));
				if (bGeneratorsUpdated || bUpdateMix)
				{
					bShowRenderStatMix = bUpdateMix;
				}
				else
				{
					// Off by default as there may be many of these
					ResetGeneratorStats();
					bShowRenderStatMix = 1;
				}

				FParse::Value(Stream, TEXT("BusFilter"), BusStringFilter);
				FParse::Value(Stream, TEXT("GeneratorFilter"), GeneratorStringFilter);
				FParse::Value(Stream, TEXT("MixFilter"), MixStringFilter);
			}
			else
			{
				bShowRenderStatMix = 1;

				// Off by default as there may be many of these
				ResetGeneratorStats();

				BusStringFilter.Reset();
				GeneratorStringFilter.Reset();
				MixStringFilter.Reset();
			}
		}

		bActive = !bActive;
		return true;
	}

	void FAudioModulationDebugger::ResetGeneratorStats()
	{
		for (TPair<FString, FGeneratorDebugInfo>& GeneratorStatPair : FilteredGeneratorsMap)
		{
			GeneratorStatPair.Value.bEnabled = false;
		}
	}

	Audio::FDeviceId FAudioModulationDebugger::GetAssociatedDeviceID()
	{
		check(IsInGameThread());
		
		if (!bActive)
		{
			bActive = true;
		}
		
		return AssociatedDeviceId;
	}

	void FAudioModulationDebugger::GetControlBusDebugInfo(TArray<FControlBusDebugInfo>& OutDebugInfo)
	{
		check(IsInGameThread());
		
		if (!bActive)
		{
			bActive = true;
			return;
		}
		
		OutDebugInfo = UnfilteredBuses;
	}

	void FAudioModulationDebugger::GetControlBusMixDebugInfo(TArray<FControlBusMixDebugInfo>& OutDebugInfo)
	{
		check(IsInGameThread());
		
		if (!bActive)
		{
			bActive = true;
			return;
		}
		
		OutDebugInfo = UnfilteredMixes;
	}
} // namespace AudioModulation
#endif // !UE_BUILD_SHIPPING
#endif // WITH_AUDIOMODULATION
