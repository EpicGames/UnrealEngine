// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Build.h"

#if WITH_AUDIOMODULATION
#if !UE_BUILD_SHIPPING
#include "IAudioModulationDebugDataProvider.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "SoundModulationGenerator.h"
#include "SoundModulationProxy.h"
#include "Templates/SharedPointer.h"


namespace AudioModulation
{
	// Forward Declarations
	struct FReferencedProxies;
	
	struct FGeneratorSort
	{
		FORCEINLINE bool operator()(const FString& A, const FString& B) const
		{
			return A < B;
		}
	};

	struct FGeneratorDebugInfo
	{
		FGeneratorDebugInfo()
		{
		}

		FGeneratorDebugInfo(const TArray<FString>& InCategories)
			: Categories(InCategories)
		{
		}

		bool bEnabled = false;
		TArray<FString> Categories;

		FString NameFilter;

		using FInstanceValues = TArray<FString>;
		TArray<FInstanceValues> FilteredInstances;
	};

	class FAudioModulationDebugger : public IDebugDataProvider, public TSharedFromThis<FAudioModulationDebugger, ESPMode::ThreadSafe>
	{
	public:
		FAudioModulationDebugger();
		virtual ~FAudioModulationDebugger();

		void UpdateDebugData(double InElapsed, const FReferencedProxies& InRefProxies, Audio::FDeviceId DeviceId);
		void SetDebugBusFilter(const FString* InNameFilter);
		void SetDebugMatrixEnabled(bool bInIsEnabled);
		void SetDebugMixFilter(const FString* InNameFilter);
		void SetDebugGeneratorsEnabled(bool bInIsEnabled);
		void SetDebugGeneratorFilter(const FString* InFilter);
		void SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled);
		void SetDebugActiveMixesEnabled(bool bInIsEnabled);
		void SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled);
		bool OnPostHelp(FCommonViewportClient& ViewportClient, const TCHAR* Stream);
		int32 OnRenderStat(FCanvas& Canvas, int32 X, int32 Y, const UFont& Font);
		bool OnToggleStat(FCommonViewportClient& ViewportClient, const TCHAR* Stream);

		void ResetGeneratorStats();

		//IdebugDataProviderInterface
		virtual Audio::FDeviceId GetAssociatedDeviceID() override;
		virtual void GetControlBusDebugInfo(TArray<FControlBusDebugInfo>& OutDebugInfo) override;
		virtual void GetControlBusMixDebugInfo(TArray<FControlBusMixDebugInfo>& OutDebugInfo) override;
		

	private:
		uint8 bActive : 1;
		uint8 bShowRenderStatMix : 1;
		uint8 bShowGenerators : 1;
		uint8 bEnableAllGenerators : 1;
		uint8 bShowActiveMixes : 1;
		uint8 bShowGlobalMixes : 1;

		Audio::FDeviceId AssociatedDeviceId;

		TArray<FControlBusDebugInfo> UnfilteredBuses;
		TArray<FControlBusDebugInfo> FilteredBuses;
		TArray<FControlBusMixDebugInfo> UnfilteredMixes;
		TArray<FControlBusMixDebugInfo> FilteredMixes;

		using FGeneratorSortMap = TSortedMap<FString, FGeneratorDebugInfo, FDefaultAllocator, FGeneratorSort>;
		FGeneratorSortMap FilteredGeneratorsMap;

		TMap<FString, bool> RequestedGeneratorUpdate;

		FString BusStringFilter;
		FString GeneratorStringFilter;
		FString MixStringFilter;

		float ElapsedSinceLastUpdate;
	};

} // namespace AudioModulation
#endif // !UE_BUILD_SHIPPING
#endif // WITH_AUDIOMODULATION
