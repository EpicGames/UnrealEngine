// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationDestination.h"

#include "AudioDeviceHandle.h"
#include "AudioModulation.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioModulationDestination)


namespace AudioModulation::DestinationPrivate
{
	FAudioDeviceHandle GetAudioDevice(const UAudioModulationDestination& InDestination)
	{
		if (GEngine)
		{
			if (GEngine->UseSound())
			{
				UWorld* World = GEngine->GetWorldFromContextObject(&InDestination, EGetWorldErrorMode::ReturnNull);
				if (World && World->bAllowAudioPlayback && !World->IsNetMode(NM_DedicatedServer))
				{
					return World->GetAudioDevice();
				}
				else
				{
					return GEngine->GetMainAudioDevice();
				}
			}
		}

		return { };
	}

	FAudioModulationManager* GetModulationManager(const UAudioModulationDestination& InDestination)
	{
		FAudioDeviceHandle AudioDevice = DestinationPrivate::GetAudioDevice(InDestination);
		if (AudioDevice.IsValid() && AudioDevice->IsModulationPluginEnabled())
		{
			if (IAudioModulationManager* ModulationInterface = AudioDevice->ModulationInterface.Get())
			{
				return static_cast<FAudioModulationManager*>(ModulationInterface);
			}
		}

		return nullptr;
	}
} // AudioModulation::DestinationPrivate

bool UAudioModulationDestination::ClearModulator()
{
	if (Modulator)
	{
		Modulator = nullptr;
		Destination.SetModulators(TSet<const USoundModulatorBase*>{ });
		return true;
	}

	return false;
}

const USoundModulatorBase* UAudioModulationDestination::GetModulator() const
{
	return Modulator;
}

float UAudioModulationDestination::GetValue() const
{
	using namespace AudioModulation;

	if (Modulator)
	{
		if (FAudioModulationManager* Modulation = DestinationPrivate::GetModulationManager(*this))
		{
			return Modulation->GetModulatorValueThreadSafe(Modulator->GetUniqueID());
		}
	}

	return 1.0f;
}

void UAudioModulationDestination::PostInitProperties()
{
	using namespace AudioModulation;

	Super::PostInitProperties();

	if (UAudioModulationDestination::StaticClass()->GetDefaultObject() == this)
	{
		return;
	}

	FAudioDeviceHandle AudioDevice = DestinationPrivate::GetAudioDevice(*this);
	if (AudioDevice.IsValid())
	{
		constexpr bool bIsBuffered = false;
		Destination.Init(AudioDevice.GetDeviceID(), bIsBuffered);
	}
}

bool UAudioModulationDestination::SetModulator(const USoundModulatorBase* InModulator)
{
	if (InModulator == Modulator)
	{
		return true;
	}

	if (InModulator)
	{
		Modulator = InModulator;
		Destination.SetModulators(TSet<const USoundModulatorBase*>{ InModulator });
		return true;
	}

	return ClearModulator();
}
