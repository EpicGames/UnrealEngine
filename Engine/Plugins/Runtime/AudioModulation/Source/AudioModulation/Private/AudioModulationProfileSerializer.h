// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioModulationLogging.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "SoundModulationValue.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


namespace AudioModulation
{
	class FProfileSerializer
	{
		static bool StageValueToFloat(const FConfigSection& InSection, const FName& InMember, float& OutValue)
		{
			if (const FConfigValue* Value = InSection.Find(InMember))
			{
				OutValue = FCString::Atof(*Value->GetValue());
				return true;
			}

			return false;
		}

		static FString GetConfigDir()
		{
			return FPaths::GeneratedConfigDir() + TEXT("AudioModulation");
		}

		static bool GetConfigDirectory(const FString& InDir, bool bCreateDirIfMissing)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!bCreateDirIfMissing && PlatformFile.DirectoryExists(*InDir))
			{
				return false;
			}

			TArray<FString> DirArray;
			InDir.ParseIntoArray(DirArray, TEXT("/"));

			FString SubPath = GetConfigDir();

			if (!PlatformFile.CreateDirectory(*SubPath))
			{
				return false;
			}

			for (FString& Dir : DirArray)
			{
				SubPath /= Dir;
				if (!PlatformFile.CreateDirectory(*SubPath))
				{
					return false;
				}
			}

			return true;
		}

		static bool GetMixProfileConfigPath(const FString& Name, const FString& Path, const int32 InProfileIndex, bool bInCreateDirIfMissing, FString& OutMixProfileConfig)
		{
			FString ConfigDir = FPaths::GetPath(Path);
			FString ConfigPath = GetConfigDir() + ConfigDir / Name;

			ConfigPath += InProfileIndex > 0
				? FString::Printf(TEXT("%i.ini"), InProfileIndex)
				: FString(TEXT(".ini"));

			if (!GConfig)
			{
				UE_LOGF(LogAudioModulation, Error,
					"Failed to find mix profile '%ls' config file. Config cache not initialized.",
					*Path);
				return false;
			}

			if (!GetConfigDirectory(ConfigDir, bInCreateDirIfMissing))
			{
				UE_LOGF(LogAudioModulation, Error, "Failed to %ls mix profile '%ls' config directory.",
					bInCreateDirIfMissing ? TEXT("create") : TEXT("find"),
					*Path);
				return false;
			}

			OutMixProfileConfig = ConfigPath;
			return true;
		}

	public:
		static bool Serialize(const USoundControlBusMix& InBusMix, const int32 InProfileIndex, const FString* InMixPathOverride = nullptr)
		{
			check(IsInGameThread());

			const FString Name = InMixPathOverride ? FPaths::GetBaseFilename(*InMixPathOverride) : InBusMix.GetName();
			const FString Path = InMixPathOverride ? *InMixPathOverride : InBusMix.GetPathName();

			FString ConfigPath;
			if (!GetMixProfileConfigPath(Name, Path, InProfileIndex, true /* bCreateDir */, ConfigPath))
			{
				return false;
			}

			// this will create and try to read in if not already loaded
			FConfigFile* ConfigFile = GConfig->Find(ConfigPath);
			if (!ConfigFile)
			{
				ConfigFile = &(GConfig->Add(ConfigPath, FConfigFile()));
			}

			ConfigFile->Reset();

			for (const FSoundControlBusMixStage& Stage : InBusMix.MixStages)
			{
				if (Stage.Bus)
				{
					FString SectionName = Stage.Bus->GetPathName();
					ConfigFile->SetFloat(*SectionName, *GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, AttackTime).ToString(), Stage.Value.AttackTime);
					ConfigFile->SetFloat(*SectionName, *GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, ReleaseTime).ToString(), Stage.Value.ReleaseTime);
					ConfigFile->SetFloat(*SectionName, *GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetValue).ToString(), Stage.Value.TargetValue);
				}
			}
			ConfigFile->Dirty = true;

			UE_LOGF(LogAudioModulation, Display, "Serialized mix to '%ls'", *Path);
			GConfig->Flush(false, ConfigPath);

			return true;
		}

		static bool Deserialize(int32 InProfileIndex, USoundControlBusMix& InOutBusMix, const FString* InMixPathOverride = nullptr)
		{
			check(IsInGameThread());

			const FString Name = InMixPathOverride ? FPaths::GetBaseFilename(*InMixPathOverride) : InOutBusMix.GetName();
			const FString Path = InMixPathOverride ? *InMixPathOverride : InOutBusMix.GetPathName();

			FString ConfigPath;
			if (!GetMixProfileConfigPath(Name, Path, InProfileIndex, false /* bCreateDir */, ConfigPath))
			{
				return false;
			}

			FConfigFile* ConfigFile = GConfig->Find(ConfigPath);
			if (!ConfigFile)
			{
				UE_LOGF(LogAudioModulation, Warning, "Config '%ls' not found. Mix '%ls' not loaded from config profile.", *ConfigPath , *Name);
				return false;
			}

			bool bMarkDirty = false;
			TSet<FString> SectionsProcessed;
			TArray<FSoundControlBusMixStage>& Stages = InOutBusMix.MixStages;
			for (int32 i = Stages.Num() - 1; i >= 0; --i)
			{
				FSoundControlBusMixStage& Stage = Stages[i];
				if (Stage.Bus)
				{
					FString PathName = Stage.Bus->GetPathName();
					if (const FConfigSection* ConfigSection = ConfigFile->FindSection(PathName))
					{
						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, AttackTime), Stage.Value.AttackTime);
						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, ReleaseTime), Stage.Value.ReleaseTime);
						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetValue), Stage.Value.TargetValue);

#if WITH_EDITORONLY_DATA
						if (USoundModulationParameter* Parameter = Stage.Bus->Parameter)
						{
							const float UnitValue = Parameter->ConvertNormalizedToUnit(Stage.Value.TargetValue);
							Stage.Value.TargetUnitValue = UnitValue;
						}
						else
						{
							Stage.Value.TargetUnitValue = Stage.Value.TargetValue;
						}
#endif // WITH_EDITORONLY_DATA

						SectionsProcessed.Emplace(MoveTemp(PathName));
						bMarkDirty = true;
					}
					else
					{
						Stages.RemoveAt(i);
					}
				}
				else
				{
					Stages.RemoveAt(i);
				}
			}

			TArray<FString> ConfigSectionNames;
			ConfigFile->GetKeys(ConfigSectionNames);
			for (FString& SectionName : ConfigSectionNames)
			{
				if (!SectionsProcessed.Contains(SectionName))
				{
					const FConfigSection* ConfigSection = ConfigFile->FindSection(SectionName);
					UObject* BusObj = FSoftObjectPath(SectionName).TryLoad();
					if (USoundControlBus* Bus = Cast<USoundControlBus>(BusObj))
					{
						FSoundControlBusMixStage Stage;
						Stage.Bus = Bus;

						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, AttackTime),  Stage.Value.AttackTime);
						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, ReleaseTime), Stage.Value.ReleaseTime);
						StageValueToFloat(*ConfigSection, GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetValue), Stage.Value.TargetValue);

						Stages.Emplace(MoveTemp(Stage));

						bMarkDirty = true;
					}
					else
					{
						UE_LOGF(LogAudioModulation, Warning,
							"Bus missing or invalid type at '%ls'. Profile '%ls' stage not added/updated for mix '%ls'",
							*SectionName,
							*ConfigPath,
							*Path);
					}
				}
			}

			if (bMarkDirty)
			{
				InOutBusMix.MarkPackageDirty();
			}

			UE_LOGF(LogAudioModulation, Display, "Deserialized mix from '%ls'", *ConfigPath);
			return true;
		}
	};
} // namespace AudioModulation
