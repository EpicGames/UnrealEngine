// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationSettings.h"

#include "AudioModulationLogging.h"
#include "IAudioModulation.h"
#include "SoundModulationParameter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioModulationSettings)

#if WITH_EDITOR
void UAudioModulationSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UAudioModulationSettings, Parameters))
		{
			UE_LOGF(LogAudioModulation, Display, "Reloading Parameters From Modulation Developer Settings...");
			RegisterParameters();
		}
	}
}
#endif // WITH_EDITOR

void UAudioModulationSettings::RegisterParameters() const
{
	Audio::UnregisterAllModulationParameters();

	for (const FSoftObjectPath& Path : Parameters)
	{
		if (USoundModulationParameter* Param = Cast<USoundModulationParameter>(Path.TryLoad()))
		{
			Audio::FModulationParameter NewParam = Param->CreateParameter();
			Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
			UE_LOGF(LogAudioModulation, Display, "Initialized Audio Modulation Parameter '%ls'", *NewParam.ParameterName.ToString());
		}
		else
		{
			const FString& ParamName = Path.GetAssetName();
			UE_LOGF(LogAudioModulation, Error, "Failed to load parameter at '%ls': Missing asset or invalid type.", *ParamName);

			// Create a parameter procedurally instead. These parameters are all required by Audio Mixer Sources.
			if (ParamName == "HPFCutoffFrequency")
			{
				Audio::FModulationParameter NewParam = USoundModulationParameterHPFFrequency::CreateDefaultParameter();
				NewParam.ParameterName = "HPFCutoffFrequency";
				Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
				UE_LOGF(LogAudioModulation, Error, "Created temporary HPFCutoffFrequency parameter to use instead.");
			}
			else if (ParamName == "LPFCutoffFrequency")
			{
				Audio::FModulationParameter NewParam = USoundModulationParameterLPFFrequency::CreateDefaultParameter();
				NewParam.ParameterName = "LPFCutoffFrequency";
				Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
				UE_LOGF(LogAudioModulation, Error, "Created temporary LPFCutoffFrequency parameter to use instead.");
			}
			else if (ParamName == "Pitch")
			{
				// The Pitch parameter is a specialized Bipolar parameter, so we have to give it extra data and change the unit display name
				Audio::FModulationParameter NewParam = USoundModulationParameterBipolar::CreateDefaultParameter(24.0f);
				NewParam.ParameterName = "Pitch";
				Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
				UE_LOGF(LogAudioModulation, Error, "Created temporary Pitch parameter to use instead.");
			}
			else if (ParamName == "Volume")
			{
				// TODO: Similar to pitch, send -60 dB to the function
				Audio::FModulationParameter NewParam = USoundModulationParameterVolume::CreateDefaultParameter(-60.0f);
				NewParam.ParameterName = "Volume";
				Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
				UE_LOGF(LogAudioModulation, Error, "Created temporary Volume parameter to use instead.");
			}
		}
	}
}

TObjectPtr<USoundModulationParameter> UAudioModulationSettings::GetModulationParameter(const FString& InName) const
{
	for (const FSoftObjectPath& Path : Parameters)
	{
		if (USoundModulationParameter* Param = Cast<USoundModulationParameter>(Path.TryLoad()))
		{
			if (Param->GetName() == InName)
			{
				return Param;
			}
		}
	}

	return nullptr;
}
