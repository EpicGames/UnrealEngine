// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationStatics.h"

#include "Async/Async.h"
#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationDestination.h"
#include "AudioModulationLogging.h"
#include "AudioModulationProfileSerializer.h"
#include "AudioModulationSystem.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioModulationStatics)

#define LOCTEXT_NAMESPACE "AudioModulationStatics"


static FAutoConsoleCommand GModulationSaveMixProfile(
	TEXT("au.Modulation.SaveMixProfile"),
	TEXT("Saves modulation mix profile to the config save directory.\n"
		"Path - Path to Object\n"
		"ProfileIndex - (Optional) Index of profile (defaults to 0)"),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() < 1)
			{
				UE_LOGF(LogAudioModulation, Error, "Failed to save mix profile: Path not provided");
				return;
			}

			const FString& Path = Args[0];
			int32 ProfileIndex = 0;
			if (Args.Num() > 1)
			{
				ProfileIndex = FCString::Atoi(*Args[1]);
			}

			FSoftObjectPath ObjPath = Path;
			if (UObject* MixObj = ObjPath.TryLoad())
			{
				if (USoundControlBusMix* Mix = Cast<USoundControlBusMix>(MixObj))
				{
					UAudioModulationStatics::SaveMixToProfile(Mix, Mix, ProfileIndex);
					return;
				}
			}

			UE_LOGF(LogAudioModulation, Error, "Failed to save mix '%ls' to profile index '%i'", *Path, ProfileIndex);
		}
	)
);

static FAutoConsoleCommand GModulationLoadMixProfile(
	TEXT("au.Modulation.LoadMixProfile"),
	TEXT("Loads modulation mix profile from the config save directory.\n"
		"Path - Path to Object to load\n"
		"Activate - (Optional) Whether or not to activate/update the mix once it is loaded (default: true)."
		"ProfileIndex - (Optional) Index of profile (default: 0)"),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() < 1)
			{
				UE_LOGF(LogAudioModulation, Error, "Failed to load mix profile: Object path not provided");
				return;
			}

			const FString& Path = Args[0];
			int32 ProfileIndex = 0;
			if (Args.Num() > 1)
			{
				ProfileIndex = FCString::Atoi(*Args[1]);
			}

			bool bActivateUpdate = true;
			if (Args.Num() > 2)
			{
				bActivateUpdate = FCString::ToBool(*Args[2]);
			}

			FSoftObjectPath ObjPath = Path;
			if (UObject* MixObj = ObjPath.TryLoad())
			{
				if (USoundControlBusMix* Mix = Cast<USoundControlBusMix>(MixObj))
				{
					UAudioModulationStatics::LoadMixFromProfile(Mix, Mix, bActivateUpdate, ProfileIndex);

					if (bActivateUpdate)
					{
						UAudioModulationStatics::UpdateMixFromObject(Mix, Mix);
					}
					return;
				}
			}

			UE_LOGF(LogAudioModulation, Error, "Failed to load mix '%ls' from profile index '%i'", *Path, ProfileIndex);
		}
	)
);


UAudioModulationStatics::UAudioModulationStatics(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAudioModulationStatics::ActivateBus(const UObject* WorldContextObject, USoundControlBus* Bus)
{
	if (!Bus)
	{
		return;
	}

	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModSystem->ActivateBus(*Bus);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UAudioModulationStatics::ActivateBusMix(const UObject* WorldContextObject, USoundControlBusMix* BusMix)
{
	if (BusMix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->ActivateBusMix(*BusMix);
		}
	}
}

void UAudioModulationStatics::ActivateGenerator(const UObject* WorldContextObject, USoundModulationGenerator* Generator)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		if (Generator)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ModSystem->ActivateGenerator(*Generator);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

UWorld* UAudioModulationStatics::GetAudioWorld(const UObject* WorldContextObject)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);
	if (!World || !World->bAllowAudioPlayback || World->IsNetMode(NM_DedicatedServer))
	{
		return nullptr;
	}

	return World;
}

AudioModulation::FAudioModulationManager* UAudioModulationStatics::GetModulation(UWorld* World)
{
	FAudioDeviceHandle AudioDevice;
	if (World)
	{
		AudioDevice = World->GetAudioDevice();
	}
	else
	{
		if (GEngine)
		{
			AudioDevice = GEngine->GetMainAudioDevice();
		}
	}

	if (AudioDevice.IsValid() && AudioDevice->IsModulationPluginEnabled())
	{
		if (IAudioModulationManager* ModulationInterface = AudioDevice->ModulationInterface.Get())
		{
			return static_cast<AudioModulation::FAudioModulationManager*>(ModulationInterface);
		}
	}

	return nullptr;
}

USoundControlBus* UAudioModulationStatics::CreateBus(UObject* WorldContextObject, FName Name, USoundModulationParameter* Parameter, bool Activate)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundControlBus::StaticClass(), Name);
	USoundControlBus* NewBus = NewObject<USoundControlBus>(WorldContextObject, UniqueName);
	NewBus->Parameter = Parameter;
	NewBus->Address = Name.ToString();

	if (Activate)
	{
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ModSystem->ActivateBus(*NewBus);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	return NewBus;
}

FSoundControlBusMixStage UAudioModulationStatics::CreateBusMixStage(const UObject* WorldContextObject, USoundControlBus* Bus, float Value, float AttackTime, float ReleaseTime)
{
	FSoundControlBusMixStage MixStage;
	MixStage.Bus = Bus;
	MixStage.Value = FSoundModulationMixValue(Value, AttackTime, ReleaseTime);
	return MixStage;
}

USoundControlBusMix* UAudioModulationStatics::CreateBusMix(UObject* WorldContextObject, FName Name, TArray<FSoundControlBusMixStage> Stages, bool Activate, double Duration, bool bRetriggerOnActivation)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundControlBusMix::StaticClass(), Name);
	USoundControlBusMix* NewBusMix = NewObject<USoundControlBusMix>(WorldContextObject, UniqueName);
	for (FSoundControlBusMixStage& Stage : Stages)
	{
		if (Stage.Bus)
		{
			NewBusMix->MixStages.Emplace(Stage);
		}
		else
		{
			UE_LOGF(LogAudioModulation, Warning,
				"USoundControlBusMix '%ls' was created but bus provided is null. Stage not added to mix.",
				*Name.ToString());
		}
	}
	
	NewBusMix->Duration = Duration;
	NewBusMix->bRetriggerOnActivation = bRetriggerOnActivation;

	if (Activate)
	{
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->ActivateBusMix(*NewBusMix);
		}
	}

	return NewBusMix;
}

USoundModulationParameter* UAudioModulationStatics::CreateModulationParameter(UObject* WorldContextObject, FName Name, TSubclassOf<USoundModulationParameter> ParamClass, float DefaultValue)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundModulationParameter::StaticClass(), Name);
	USoundModulationParameter* NewParam = NewObject<USoundModulationParameter>(WorldContextObject, ParamClass, UniqueName);

	if (NewParam)
	{
		NewParam->Settings.ValueNormalized = DefaultValue;
	}

	return NewParam;
}

USoundModulationGeneratorEnvelopeFollower* UAudioModulationStatics::CreateEnvelopeFollowerGenerator(UObject* WorldContextObject, FName Name, FEnvelopeFollowerGeneratorParams Params)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundModulationGeneratorEnvelopeFollower::StaticClass(), Name);
	USoundModulationGeneratorEnvelopeFollower* NewGenerator = NewObject<USoundModulationGeneratorEnvelopeFollower>(WorldContextObject, USoundModulationGeneratorEnvelopeFollower::StaticClass(), UniqueName);

	if (NewGenerator)
	{
		NewGenerator->Params = Params;
	}

	return NewGenerator;
}

USoundModulationGeneratorLFO* UAudioModulationStatics::CreateLFOGenerator(UObject* WorldContextObject, FName Name, FSoundModulationLFOParams Params)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundModulationGeneratorLFO::StaticClass(), Name);
	USoundModulationGeneratorLFO* NewGenerator = NewObject<USoundModulationGeneratorLFO>(WorldContextObject, USoundModulationGeneratorLFO::StaticClass(), UniqueName);

	if (NewGenerator)
	{
		NewGenerator->Params = Params;
	}

	return NewGenerator;
}

USoundModulationGeneratorADEnvelope* UAudioModulationStatics::CreateADEnvelopeGenerator(UObject* WorldContextObject, FName Name, const FSoundModulationADEnvelopeParams& Params)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, USoundModulationGeneratorADEnvelope::StaticClass(), Name);
	USoundModulationGeneratorADEnvelope* NewGenerator = NewObject<USoundModulationGeneratorADEnvelope>(WorldContextObject, USoundModulationGeneratorADEnvelope::StaticClass(), UniqueName);

	if (NewGenerator)
	{
		NewGenerator->Params = Params;
	}

	return NewGenerator;
}

UAudioModulationDestination* UAudioModulationStatics::CreateModulationDestination(UObject* WorldContextObject, FName Name, USoundModulatorBase* Modulator)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (!World)
	{
		return nullptr;
	}

	FName UniqueName = MakeUniqueObjectName(WorldContextObject, UAudioModulationDestination::StaticClass(), Name);
	UAudioModulationDestination* NewDestination = NewObject<UAudioModulationDestination>(WorldContextObject, UAudioModulationDestination::StaticClass(), UniqueName);
	if (NewDestination)
	{
		NewDestination->SetModulator(Modulator);
		return NewDestination;
	}

	return nullptr;
}

void UAudioModulationStatics::DeactivateBus(const UObject* WorldContextObject, USoundControlBus* Bus)
{
	if (Bus)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ModSystem->DeactivateBus(*Bus);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void UAudioModulationStatics::DeactivateBusMix(const UObject* WorldContextObject, USoundControlBusMix* BusMix)
{
	if (BusMix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->DeactivateBusMix(*BusMix);
		}
	}
}

void UAudioModulationStatics::DeactivateBusMixByID(const UObject* WorldContextObject, uint32 MixID)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		ModSystem->DeactivateBusMixByID(MixID);
	}
}

void UAudioModulationStatics::DeactivateGenerator(const UObject* WorldContextObject, USoundModulationGenerator* Generator)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		if (Generator)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ModSystem->DeactivateGenerator(*Generator);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

bool UAudioModulationStatics::IsControlBusMixActive(const UObject* WorldContextObject, USoundControlBusMix* Mix)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		if (Mix)
		{
			return ModSystem->IsBusMixActive(*Mix);
		}
	}
	return false;
}

void UAudioModulationStatics::SaveMixToProfile(const UObject* WorldContextObject, USoundControlBusMix* BusMix, int32 ProfileIndex)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		if (BusMix)
		{
			return ModSystem->SaveMixToProfile(*BusMix, ProfileIndex);
		}
	}
}

TArray<FSoundControlBusMixStage> UAudioModulationStatics::LoadMixFromProfile(const UObject* WorldContextObject, USoundControlBusMix* BusMix, bool bActivate, int32 ProfileIndex)
{
	if (BusMix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			if (bActivate)
			{
				ActivateBusMix(WorldContextObject, BusMix);
			}
			return ModSystem->LoadMixFromProfile(ProfileIndex, *BusMix);
		}
	}

	return TArray<FSoundControlBusMixStage>();
}

void UAudioModulationStatics::UpdateMix(const UObject* WorldContextObject, USoundControlBusMix* Mix, TArray<FSoundControlBusMixStage> Stages, float InFadeTime, double Duration, bool bRetriggerOnActivation)
{
	if (Mix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			// UObject representation is not updated in this form of the call as doing so from
			// PIE can result in an unstable state where UObject is modified but not properly dirtied.
			ModSystem->UpdateMix(Stages, *Mix, false /* bUpdateObject */, InFadeTime, Duration, bRetriggerOnActivation);
		}
	}
}

USoundControlBusMix* UAudioModulationStatics::CreateBusMixFromValue(const UObject* WorldContextObject, FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime, float ReleaseTime, bool bActivate)
{
	UWorld* World = GetAudioWorld(WorldContextObject);

	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		// Outer the new bus mix to the resolved world so its lifetime is bound to that world.
		// Without this it would land in the transient package and survive PIE/world teardown,
		// continuing to drive bus values past the point the caller expects it to deactivate.
		if (USoundControlBusMix* NewMix = ModSystem->CreateBusMixFromValue(Name, Buses, Value, AttackTime, ReleaseTime, World))
		{
			if (bActivate)
			{
				ModSystem->ActivateBusMix(*NewMix);
			}

			return NewMix;
		}
	}

	return nullptr;
}

void UAudioModulationStatics::SetGlobalBusMixValue(const UObject* WorldContextObject, USoundControlBus* Bus, float Value, float FadeTime)
{
	if (Bus)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->SetGlobalBusMixValue(*Bus, Value, FadeTime);
		}
	}
}

void UAudioModulationStatics::ClearGlobalBusMixValue(const UObject* WorldContextObject, USoundControlBus* Bus, float FadeTime)
{
	if (Bus)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->ClearGlobalBusMixValue(*Bus, FadeTime);
		}
	}
}

void UAudioModulationStatics::ClearAllGlobalBusMixValues(const UObject* WorldContextObject, float FadeTime)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		ModSystem->ClearAllGlobalBusMixValues(FadeTime);
	}
}

void UAudioModulationStatics::DeactivateAllBusMixes(const UObject* WorldContextObject)
{
	UWorld* World = GetAudioWorld(WorldContextObject);
	if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
	{
		ModSystem->DeactivateAllBusMixes();
	}
}

void UAudioModulationStatics::UpdateMixByFilter(
	const UObject* WorldContextObject,
	USoundControlBusMix* Mix,
	FString AddressFilter,
	TSubclassOf<USoundModulationParameter> ParamClassFilter,
	USoundModulationParameter* ParamFilter,
	float Value,
	float FadeTime)
{
	if (Mix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			// UObject representation is not updated in this form of the call as doing so from
			// PIE can result in an unstable state where UObject is modified but not properly dirtied.
			ModSystem->UpdateMixByFilter(AddressFilter, ParamClassFilter, ParamFilter, Value, FadeTime, *Mix, false /* bUpdateObject */);
		}
	}
}

void UAudioModulationStatics::UpdateMixFromObject(const UObject* WorldContextObject, USoundControlBusMix* Mix, float InFadeTime)
{
	if (Mix)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->UpdateMix(*Mix, InFadeTime);
		}
	}
}

void UAudioModulationStatics::UpdateModulator(const UObject* WorldContextObject, USoundModulatorBase* Modulator)
{
	if (Modulator)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			ModSystem->UpdateModulator(*Modulator);
		}
	}
}

float UAudioModulationStatics::GetModulatorValue(const UObject* WorldContextObject, USoundModulatorBase* Modulator)
{	
	if (Modulator)
	{
		UWorld* World = GetAudioWorld(WorldContextObject);
		if (AudioModulation::FAudioModulationManager* ModSystem = GetModulation(World))
		{
			return ModSystem->GetModulatorValueThreadSafe(Modulator->GetUniqueID());
		}
	}

	return 1.0f;
}

TSet<USoundModulatorBase*> UAudioModulationStatics::GetModulatorsFromDestination(const FSoundModulationDestinationSettings& Destination)
{
	TSet<USoundModulatorBase*> Modulators;
	for (const TObjectPtr<USoundModulatorBase>& Modulator : Destination.Modulators)
	{
		Modulators.Add(Modulator.Get());
	}

	return Modulators;
}


#undef LOCTEXT_NAMESPACE

