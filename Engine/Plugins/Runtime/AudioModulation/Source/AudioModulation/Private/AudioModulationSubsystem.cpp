// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationSubsystem.h"

#include "AudioDevice.h"
#include "AudioModulationSystem.h"
#include "Misc/App.h"

static FAutoConsoleCommand CVarAudioModulationSetParameterClampOverride(
	TEXT("au.Modulation.SetParameterClampOverride"),
	TEXT("Sets whether the given parameter should be clamped to the normalized [0,1] range. Overrides the UObject setting, but not the ForceClampAllModulationValues CVar. Arguments:\n"
		"Parameter - The parameter to set the clamp status of.\n"
		"Clamp (Optional, Default: 1) - Whether the parameter should be clamped. 1 = clamped, 0 = unclamped.\n"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1)
		{
			return;
		}

		bool bShouldClamp = true;
		if (Args.Num() > 1)
		{
			if (FCString::IsNumeric(*Args[1]))
			{
				bShouldClamp = static_cast<bool>(FCString::Atoi(*Args[1]));
			}
		}

		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			AudioDeviceManager->IterateOverAllDevices([ParamName = Args[0], bShouldClamp](Audio::FDeviceId Id, FAudioDevice* AudioDevice )
			{
				if (!AudioDevice)
				{
					return;
				}
				
				if (UAudioModulationSubsystem* Subsystem = AudioDevice->GetSubsystem<UAudioModulationSubsystem>())
				{
					Subsystem->SetModulationParameterClampOverride(ParamName, bShouldClamp);
				}
			});
		}
	}));

static FAutoConsoleCommand CVarAudioModulationClearParameterClampOverride(
	TEXT("au.Modulation.ClearParameterClampOverride"),
	TEXT("Removes any override for the given parameter set by SetParameterClampOverride.\n"
		"Parameter - The parameter to clear the override of.\n"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1)
		{
			return;
		}

			
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			AudioDeviceManager->IterateOverAllDevices([ParamName = Args[0]](Audio::FDeviceId Id, FAudioDevice* AudioDevice )
			{
				if (!AudioDevice)
				{
					return;
				}
				
				if (UAudioModulationSubsystem* Subsystem = AudioDevice->GetSubsystem<UAudioModulationSubsystem>())
				{
					Subsystem->ClearModulationParameterClampOverride(ParamName);
				}
			});
		}
	}));

static FAutoConsoleCommand CVarAudioModulationClearAllParameterClampOverrides(
	TEXT("au.Modulation.ClearAllParameterClampOverrides"),
	TEXT("Removes all modulation parameter clamp overrides. This does not remove the global clamp set by 'au.Modulation.ForceClampAllModulationValues'. \n"),
		FConsoleCommandDelegate::CreateLambda([]()
	{
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			AudioDeviceManager->IterateOverAllDevices([](Audio::FDeviceId Id, FAudioDevice* AudioDevice )
			{
				if (!AudioDevice)
				{
					return;
				}
				
				if (UAudioModulationSubsystem* Subsystem = AudioDevice->GetSubsystem<UAudioModulationSubsystem>())
				{
					Subsystem->ClearAllModulationParameterClampOverrides();
				}
			});
		}
	}));

bool UAudioModulationSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return Super::ShouldCreateSubsystem(Outer) && FApp::CanEverRenderAudio();
}

void UAudioModulationSubsystem::Deinitialize()
{
	ModulationParameterClampOverrides.Empty();
}

void UAudioModulationSubsystem::SetModulationParameterClampOverride(const FString& ParameterName, bool bClamp) const
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Audio::SetModulationParameterClampOverride(FName(*ParameterName), bClamp);
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}

void UAudioModulationSubsystem::ClearModulationParameterClampOverride(const FString& ParameterName) const
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Audio::ClearModulationParameterClampOverride(FName(*ParameterName));
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}

void UAudioModulationSubsystem::ClearAllModulationParameterClampOverrides() const
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Audio::ClearAllModulationParameterClampOverrides();
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}
