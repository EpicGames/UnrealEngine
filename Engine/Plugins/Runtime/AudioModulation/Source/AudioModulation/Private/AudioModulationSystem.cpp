// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationSystem.h"
#include "UObject/Package.h"
#include <type_traits>

#if WITH_AUDIOMODULATION

#if !UE_BUILD_SHIPPING
#include "AudioModulationDebugger.h"
#endif // !UE_BUILD_SHIPPING

#include "Async/Async.h"
#include "Audio/AudioAddressPattern.h"
#include "Audio/AudioTraceUtil.h"
#include "AudioMixerTrace.h"
#include "AudioModulationLogging.h"
#include "AudioModulationProfileSerializer.h"
#include "AudioModulationSettings.h"
#include "AudioModulationTrace.h"
#include "AudioThread.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Generators/SoundModulationLFO.h"
#include "HAL/PlatformTLS.h"
#include "IAudioModulation.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SoundControlBusProxy.h"
#include "SoundControlBusMixProxy.h"
#include "SoundModulationGeneratorProxy.h"
#include "SoundModulationPatchProxy.h"
#include "SoundModulationProxy.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"

DECLARE_DWORD_COUNTER_STAT(TEXT("Bus Count"),	STAT_AudioModulationBusCount, STATGROUP_AudioModulation)
DECLARE_DWORD_COUNTER_STAT(TEXT("Generator Count"),	STAT_AudioModulationGeneratorCount, STATGROUP_AudioModulation)
DECLARE_DWORD_COUNTER_STAT(TEXT("Mix Count"),	STAT_AudioModulationMixCount, STATGROUP_AudioModulation)
DECLARE_DWORD_COUNTER_STAT(TEXT("Patch Count"), STAT_AudioModulationPatchCount, STATGROUP_AudioModulation)
DECLARE_DWORD_COUNTER_STAT(TEXT("Render Queue Commands Processed"), STAT_AudioModulationProcQueueCount, STATGROUP_AudioModulation)

#if UE_AUDIO_PROFILERTRACE_ENABLED

// Mirrors FModulatorTraceMessageBase: Engine/Plugins/Runtime/AudioModulationInsights/Source/AudioModulationInsights/Private/Messages/AudioModulationTraceMessages.h
#define MODULATOR_TRACE_MESSAGE_BASE \
	UE_TRACE_EVENT_FIELD(uint32, DeviceId) \
	UE_TRACE_EVENT_FIELD(uint64, Timestamp) \
	UE_TRACE_EVENT_FIELD(Audio::FModulatorId, ModulatorId)

// Mirrors FActivateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulationInsights/Source/AudioModulationInsights/Private/Messages/AudioModulationTraceMessages.h
// Note: This data doesn't change during the lifetime of the Modulator
UE_TRACE_EVENT_BEGIN(Audio, ActivateModulatorTraceMessage)
	MODULATOR_TRACE_MESSAGE_BASE
	UE_TRACE_EVENT_FIELD(Audio::FModulatorTypeId, ModulatorType)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

// Mirrors FUpdateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulationInsights/Source/AudioModulationInsights/Private/Messages/AudioModulationTraceMessages.h
// Note: This data can change during the lifetime of the Modulator
UE_TRACE_EVENT_BEGIN(Audio, UpdateModulatorTraceMessage)
	MODULATOR_TRACE_MESSAGE_BASE
	UE_TRACE_EVENT_FIELD(float, ModulatorValue)
	UE_TRACE_EVENT_FIELD(bool, IsBypassed)
	UE_TRACE_EVENT_FIELD(Audio::FModulatorId[], ContributingModulatorIds)
	UE_TRACE_EVENT_FIELD(float[], ContributingModulatorValues)
UE_TRACE_EVENT_END()

// Mirrors FDeactivateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulationInsights/Source/AudioModulationInsights/Private/Messages/AudioModulationTraceMessages.h
UE_TRACE_EVENT_BEGIN(Audio, DeactivateModulatorTraceMessage)
	MODULATOR_TRACE_MESSAGE_BASE
UE_TRACE_EVENT_END()

#endif // UE_AUDIO_PROFILERTRACE_ENABLED

namespace AudioModulation
{
	enum class EModulatorType : Audio::FModulatorTypeId
	{
		Patch,
		Bus,
		Generator,

		COUNT
	};

	FAudioModulationSystem::~FAudioModulationSystem()
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FAudioModulationSystem::Initialize(const FAudioPluginInitializationParams& InitializationParams)
	{
#if !UE_BUILD_SHIPPING
		Debugger = MakeShared<FAudioModulationDebugger>();
#endif // !UE_BUILD_SHIPPING

#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.AddRaw(this, &FAudioModulationSystem::OnTraceStarted);

		// Note: Enabling/Disabling plugins require restarting the editor, so we only need to determine a plugins state once here within Initialize
		const TSharedPtr<const IPlugin> AudioModulationInsightsPlugin = IPluginManager::Get().FindPlugin("AudioModulationInsights");
		bIsAudioModulationInsightsPluginEnabled = (AudioModulationInsightsPlugin.IsValid() && AudioModulationInsightsPlugin->IsEnabled());
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		AudioDeviceId = InitializationParams.AudioDevicePtr->DeviceID;
	}

	void FAudioModulationSystem::OnAuditionEnd()
	{
		DeactivateAllBusMixes();
	}

#if !UE_BUILD_SHIPPING
	bool FAudioModulationSystem::OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		check(IsInGameThread());
		return ViewportClient ? Debugger->OnPostHelp(*ViewportClient, Stream) : true;
	}

	int32 FAudioModulationSystem::OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		check(IsInGameThread());
		return Canvas ? Debugger->OnRenderStat(*Canvas, X, Y, Font) : Y;
	}

	bool FAudioModulationSystem::OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		check(IsInGameThread());
		return ViewportClient ? Debugger->OnToggleStat(*ViewportClient, Stream) : true;
	}
#endif // !UE_BUILD_SHIPPING

	void FAudioModulationSystem::ActivateBus(const USoundControlBus& InBus)
	{
		RunCommandOnProcessingThread([this, Settings = FControlBusSettings(InBus)]() mutable
		{
			FBusHandle BusHandle = FBusHandle::Create(MoveTemp(Settings), RefProxies.Buses, *this);
			ManuallyActivatedBuses.Add(MoveTemp(BusHandle));
		});
	}

	void FAudioModulationSystem::ActivateBusMix(FModulatorBusMixSettings&& InSettings)
	{
		ActiveBusMixIds.Add(InSettings.GetId());

		RunCommandOnProcessingThread([this, Settings = MoveTemp(InSettings)]() mutable
		{
			FBusMixHandle BusMixHandle = FBusMixHandle::Get(Settings.GetId(), RefProxies.BusMixes);
			if (BusMixHandle.IsValid())
			{
				BusMixHandle.FindProxy().SetMixDataAndEnable(MoveTemp(Settings));
			}
			else
			{
				BusMixHandle = FBusMixHandle::Create(MoveTemp(Settings), RefProxies.BusMixes, *this);
			}

			ManuallyActivatedBusMixes.Add(MoveTemp(BusMixHandle));
		});
	}

	void FAudioModulationSystem::ActivateBusMix(const USoundControlBusMix& InBusMix)
	{
		ActivateBusMix(FModulatorBusMixSettings(InBusMix));
	}

	void FAudioModulationSystem::ActivateGenerator(const USoundModulationGenerator& InGenerator)
	{
		RunCommandOnProcessingThread([this, Settings = FModulationGeneratorSettings(InGenerator)]() mutable
		{
			FGeneratorHandle GeneratorHandle = FGeneratorHandle::Get(Settings.GetId(), RefProxies.Generators);
			if (GeneratorHandle.IsValid())
			{
				ManuallyActivatedGenerators.Add(MoveTemp(GeneratorHandle));
			}
			else
			{
				GeneratorHandle = FGeneratorHandle::Create(MoveTemp(Settings), RefProxies.Generators, *this);
				GeneratorHandle.FindProxy().Init(AudioDeviceId);
			}
		});
	}

	void FAudioModulationSystem::AddReferencedObjects(FReferenceCollector& Collector)
	{
		TArray<TObjectPtr<USoundControlBusMix>> GlobalBusMixes;
		ActiveGlobalBusValueMixes.GenerateValueArray(GlobalBusMixes);
		Collector.AddReferencedObjects(GlobalBusMixes);
	}

	FString FAudioModulationSystem::GetReferencerName() const
	{
		return TEXT("FAudioModulationSystem");
	}

	bool FAudioModulationSystem::CalculateModulationValue(FModulationPatchProxy& OutProxy, float& OutValue) const
	{
		check(IsInProcessingThread());
		if (OutProxy.IsBypassed())
		{
			return false;
		}

		const float InitValue = OutValue;
		OutProxy.Update();
		OutValue = OutProxy.GetValue();
		return !FMath::IsNearlyEqual(InitValue, OutValue);
	}

	void FAudioModulationSystem::DeactivateBus(const USoundControlBus& InBus)
	{
		ClearGlobalBusMixValue(InBus);

		RunCommandOnProcessingThread([this, BusId = static_cast<FBusId>(InBus.GetUniqueID())]()
		{
			FBusHandle BusHandle = FBusHandle::Get(BusId, RefProxies.Buses);
			if (BusHandle.IsValid())
			{
				ManuallyActivatedBuses.Remove(BusHandle);
			}
		});
	}

	void FAudioModulationSystem::DeactivateBusMix(const USoundControlBusMix& InBusMix)
	{
		ActiveBusMixIds.Remove(InBusMix.GetUniqueID());
		
		RunCommandOnProcessingThread([this, BusMixId = static_cast<FBusMixId>(InBusMix.GetUniqueID())]()
		{
			FBusMixHandle MixHandle = FBusMixHandle::Get(BusMixId, RefProxies.BusMixes);
			if (MixHandle.IsValid())
			{
				FModulatorBusMixProxy& MixProxy = MixHandle.FindProxy();
				MixProxy.SetStopping();
			}
		});
	}

	void FAudioModulationSystem::DeactivateBusMixByID(const uint32 ID)
	{
		ActiveBusMixIds.Remove(ID);

		RunCommandOnProcessingThread([this, BusMixId = static_cast<FBusMixId>(ID)]()
			{
				FBusMixHandle MixHandle = FBusMixHandle::Get(BusMixId, RefProxies.BusMixes);
				if (MixHandle.IsValid())
				{
					FModulatorBusMixProxy& MixProxy = MixHandle.FindProxy();
					MixProxy.SetStopping();
				}
			});
	}

	void FAudioModulationSystem::DeactivateAllBusMixes()
	{
		ClearAllGlobalBusMixValues();

		ActiveBusMixIds.Empty();

		RunCommandOnProcessingThread([this]()
		{
			for (auto& [BusMixId, MixProxy] : RefProxies.BusMixes)
			{
				MixProxy.SetStopping();
			}
		});
	}

	void FAudioModulationSystem::DeactivateGenerator(const USoundModulationGenerator& InGenerator)
	{
		RunCommandOnProcessingThread([this, GeneratorId = static_cast<FGeneratorId>(InGenerator.GetUniqueID())]()
		{
			FGeneratorHandle GeneratorHandle = FGeneratorHandle::Get(GeneratorId, RefProxies.Generators);
			if (GeneratorHandle.IsValid())
			{
				ManuallyActivatedGenerators.Remove(GeneratorHandle);
			}
		});
	}

#if !UE_BUILD_SHIPPING
	void FAudioModulationSystem::SetDebugBusFilter(const FString* InFilter)
	{
		Debugger->SetDebugBusFilter(InFilter);
	}

	void FAudioModulationSystem::SetDebugGeneratorsEnabled(bool bInIsEnabled)
	{
		Debugger->SetDebugGeneratorsEnabled(bInIsEnabled);
	}

	void FAudioModulationSystem::SetDebugGeneratorFilter(const FString* InFilter)
	{
		Debugger->SetDebugGeneratorFilter(InFilter);
	}

	void FAudioModulationSystem::SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInEnabled)
	{
		Debugger->SetDebugGeneratorTypeFilter(InFilter, bInEnabled);
	}

	void FAudioModulationSystem::SetDebugActiveMixesEnabled(bool bInIsEnabled)
	{
		Debugger->SetDebugActiveMixesEnabled(bInIsEnabled);
	}

	void FAudioModulationSystem::SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled)
	{
		Debugger->SetDebugActiveGlobalMixesEnabled(bInIsEnabled);
	}

	void FAudioModulationSystem::SetDebugMatrixEnabled(bool bInIsEnabled)
	{
		Debugger->SetDebugMatrixEnabled(bInIsEnabled);
	}

	void FAudioModulationSystem::SetDebugMixFilter(const FString* InNameFilter)
	{
		Debugger->SetDebugMixFilter(InNameFilter);
	}
#endif // !UE_BUILD_SHIPPING

	void FAudioModulationSystem::SetGlobalBusMixValue(USoundControlBus& InBus, float InValue, float InFadeTime)
	{
		if (ActiveGlobalBusValueMixes.Contains(InBus.GetUniqueID()))
		{
			TObjectPtr<USoundControlBusMix> GlobalMix = ActiveGlobalBusValueMixes.FindRef(InBus.GetUniqueID());

			if (ensure(!GlobalMix->MixStages.IsEmpty()))
			{
				GlobalMix->MixStages[0].Value.TargetValue = InValue;
				UpdateMix(*GlobalMix, InFadeTime);

				UE_LOGF(LogAudioModulation, VeryVerbose, "GlobalBusMix for ControlBus '%ls' updated, target set to '%0.4f'.", *InBus.GetName(), InValue);
			}
		}
		else
		{
			const FName MixName(*FString::Printf(TEXT("%s_GlobalMix"), *InBus.GetName()));
			if (TObjectPtr<USoundControlBusMix> NewGlobalMix = CreateBusMixFromValue(MixName, { &InBus }, InValue, InFadeTime))
			{
				ActiveGlobalBusValueMixes.Add(InBus.GetUniqueID(), NewGlobalMix);
				UE_LOGF(LogAudioModulation, VeryVerbose, "GlobalBusMix for ControlBus '%ls' activated, target set to '%0.4f'.", *InBus.GetName(), InValue);
				ActivateBusMix(*NewGlobalMix);
			}
			else
			{
				UE_LOGF(LogAudioModulation, Warning, "Could not set Global Bus Mix value: failed to allocate new Global Bus Mix for bus %ls.", *InBus.GetName());
			}
		}

	}

	void FAudioModulationSystem::ClearGlobalBusMixValue(const USoundControlBus& InBus, float InFadeTime)
	{
		const uint32 BusID = InBus.GetUniqueID();
		if (TObjectPtr<USoundControlBusMix> GlobalMix = ActiveGlobalBusValueMixes.FindRef(BusID))
		{
			if (ensure(!GlobalMix->MixStages.IsEmpty()))
			{
				GlobalMix->MixStages[0].Value.ReleaseTime = InFadeTime;
				DeactivateBusMix(*GlobalMix);
				ActiveGlobalBusValueMixes.Remove(BusID);
				UE_LOGF(LogAudioModulation, VeryVerbose, "GlobalBusMix for ControlBus '%ls' cleared.", *InBus.GetName());
			}
		}
		else
		{
			UE_LOGF(LogAudioModulation, VeryVerbose, "GlobalBusMix for ControlBus '%ls' not active, ignoring clear request.", *InBus.GetName());
		}
	}

	void FAudioModulationSystem::ClearAllGlobalBusMixValues(float InFadeTime)
	{
		TArray<TObjectPtr<USoundControlBusMix>> GlobalBusMixes;
		ActiveGlobalBusValueMixes.GenerateValueArray(GlobalBusMixes);
		for (const TObjectPtr<USoundControlBusMix>& BusMix : GlobalBusMixes)
		{
			if (ensure(!BusMix->MixStages.IsEmpty()))
			{
				ClearGlobalBusMixValue(*BusMix->MixStages[0].Bus, InFadeTime);
			}
		}

		ActiveGlobalBusValueMixes.Reset();
	}

	bool FAudioModulationSystem::IsBusMixGlobal(FBusMixId InBusMixId) const
	{
		// Right now, anyone calling this is assuming the ID of ActiveGlobalBuxValueMixes is the Control Bus ID, not the Control Bus Mix ID.
		return ActiveGlobalBusValueMixes.Contains(InBusMixId);
	}

	USoundControlBusMix* FAudioModulationSystem::CreateBusMixFromValue(FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime, float ReleaseTime, UObject* InOuter)
	{
		// Outer to the caller's scope when supplied (typically the resolved UWorld) so the bus mix's
		// lifetime tracks that scope and BeginDestroy fires when the world tears down. The transient
		// package fallback preserves prior behavior for internal callers that GC-root the mix
		// elsewhere (e.g. SetGlobalBusMixValue keeps it alive via ActiveGlobalBusValueMixes).
		UObject* Outer = InOuter ? InOuter : static_cast<UObject*>(GetTransientPackage());
		if (TObjectPtr<USoundControlBusMix> NewGlobalMix = NewObject<USoundControlBusMix>(Outer, Name))
		{
			for (USoundControlBus* Bus : Buses)
			{
				if (Bus)
				{
					FSoundModulationMixValue MixValue;
					MixValue.TargetValue = Value;

					if (AttackTime >= 0.0f)
					{
						MixValue.AttackTime = AttackTime;
					}

					if (ReleaseTime >= 0.0f)
					{
						MixValue.ReleaseTime = ReleaseTime;
					}

					FSoundControlBusMixStage MixStage;
					MixStage.Bus = Bus;
					MixStage.Value = MixValue;

					NewGlobalMix->MixStages.Emplace(MoveTemp(MixStage));
				}
			}
			return NewGlobalMix;
		}

		return nullptr;
	}

	bool FAudioModulationSystem::GetModulatorValue(const Audio::FModulatorHandle& InModulatorHandle, float& OutValue) const
	{
		const EModulatorType ModulatorType = static_cast<EModulatorType>(InModulatorHandle.GetTypeId());

		switch (ModulatorType)
		{
			case EModulatorType::Patch:
			{
				// Direct access preferred vs through handles here as its impossible for proxies to be destroyed
				// in look-up and speed is key as this is possibly being queried often in the audio render pass.
				if (const FModulationPatchRefProxy* PatchProxy = RefProxies.Patches.Find(static_cast<FPatchId>(InModulatorHandle.GetModulatorId())))
				{
					if (!PatchProxy->IsBypassed())
					{
						OutValue = PatchProxy->GetValue();
						return true;
					}
				}
			}
			break;

			case EModulatorType::Bus:
			{
				if (const FControlBusProxy* BusProxy = RefProxies.Buses.Find(static_cast<FBusId>(InModulatorHandle.GetModulatorId())))
				{
					if (!BusProxy->IsBypassed())
					{
						OutValue = BusProxy->GetValue();
						return true;
					}
				}
			}
			break;

			case EModulatorType::Generator:
			{
				if (const FModulatorGeneratorProxy* GeneratorProxy = RefProxies.Generators.Find(static_cast<FGeneratorId>(InModulatorHandle.GetModulatorId())))
				{
					if (!GeneratorProxy->IsBypassed())
					{
						OutValue = GeneratorProxy->GetValue();
						return true;
					}
				}
			}
			break;

			default:
			{
				static_assert(static_cast<uint32>(EModulatorType::COUNT) == 3, "Possible missing modulator type coverage in switch statement");
			}
			break;
		}

		return false;
	}

	bool FAudioModulationSystem::GetModulatorValueThreadSafe(const Audio::FModulatorHandle& InModulatorHandle, float& OutValue) const
	{
		FScopeLock Lock(&ThreadSafeModValueCritSection);

		if (const float* Value = ThreadSafeModValueMap.Find(InModulatorHandle.GetModulatorId()))
		{
			OutValue = *Value;
			return true;
		}

		return false;
	}

	bool FAudioModulationSystem::GetModulatorValueThreadSafe(uint32 ModulatorID, float& OutValue) const
	{
		FScopeLock Lock(&ThreadSafeModValueCritSection);

		if (const float* Value = ThreadSafeModValueMap.Find(ModulatorID))
		{
			OutValue = *Value;
			return true;
		}

		return false;
	}

	Audio::FDeviceId FAudioModulationSystem::GetAudioDeviceId() const
	{
		return AudioDeviceId;
	}

	bool FAudioModulationSystem::IsInProcessingThread() const
	{
		return ProcessingThreadId == FPlatformTLS::GetCurrentThreadId();
	}

	void FAudioModulationSystem::ProcessModulators(const double InElapsed)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioModulationSystem::ProcessModulators);

		// The processing thread can get updated between frames. As modulation
		// processing should be first call in frame, update the threadId accordingly
		ProcessingThreadId = FPlatformTLS::GetCurrentThreadId();

		int32 CommandsProcessed = 0;
		TUniqueFunction<void()> Command;
		while (ProcessingThreadCommandQueue.Dequeue(Command))
		{
			Command();
			++CommandsProcessed;
		}

		TMap<Audio::FModulatorId, float> NewModulatorValues;

		// Update Generators (prior to bus mixing to avoid single-frame latency)
		for (TPair<FGeneratorId, FModulatorGeneratorProxy>& Pair : RefProxies.Generators)
		{
			Pair.Value.PumpCommands();
			Pair.Value.Update(InElapsed);
			NewModulatorValues.Add(static_cast<Audio::FModulatorId>(Pair.Key), Pair.Value.GetValue());
		}

		// Reset buses & refresh cached Generator
		for (TPair<FBusId, FControlBusProxy>& Pair : RefProxies.Buses)
		{
			Pair.Value.Reset();
			Pair.Value.MixGenerators();
		}

		// Update mix values and apply to prescribed buses.
		// Track bus mixes ready to remove
		TSet<FBusMixId> StoppedMixIds;
		for (TPair<FBusMixId, FModulatorBusMixProxy>& Pair : RefProxies.BusMixes)
		{
			const FModulatorBusMixProxy::EStatus LastStatus = Pair.Value.GetStatus();
			Pair.Value.Update(InElapsed, RefProxies.Buses);
			const FModulatorBusMixProxy::EStatus CurrentStatus = Pair.Value.GetStatus();

			switch (CurrentStatus)
			{
				case FModulatorBusMixProxy::EStatus::Enabled:
				case FModulatorBusMixProxy::EStatus::Stopping:
				break;

				case FModulatorBusMixProxy::EStatus::Stopped:
				{
					if (LastStatus != CurrentStatus)
					{
						UE_LOGF(LogAudioModulation, Verbose, "Audio modulation mix '%ls' stopped.", *Pair.Value.GetName().ToString());
					}
					StoppedMixIds.Add(Pair.Key);
				}
				break;

				default:
				{
					checkf(false, TEXT("Invalid or unsupported BusMix EStatus state advancement."));
				}
				break;
			}
		}

		// Destroy mixes that have stopped (must be done outside mix update
		// loop above to avoid destroying while iterating, which can occur
		// when update moves bus mix from 'stopping' status to 'stopped')
		for (const FBusMixId& MixId : StoppedMixIds)
		{
			FBusMixHandle MixHandle = FBusMixHandle::Get(MixId, RefProxies.BusMixes);

			// Expected to be valid given the fact that the proxy is available in the prior loop
			check(MixHandle.IsValid());

			// Expected to only have two references (one for transient 'MixHandle' and one in
			// ManuallyActivated set). Nothing else should be keeping mixes active.
			check(MixHandle.FindProxy().GetRefCount() == 2);

			ManuallyActivatedBusMixes.Remove(MoveTemp(MixHandle));
		}

		for (TPair<FPatchId, FModulationPatchRefProxy>& Pair : RefProxies.Patches)
		{
			FModulationPatchRefProxy& PatchProxy = Pair.Value;
			if (!PatchProxy.IsBypassed())
			{
				PatchProxy.Update();
				NewModulatorValues.Add(static_cast<Audio::FModulatorId>(Pair.Key), PatchProxy.GetValue());
			}
		}

		for (const TPair<FBusId, FControlBusProxy>& Pair : RefProxies.Buses)
		{
			NewModulatorValues.Add(static_cast<Audio::FModulatorId>(Pair.Key), Pair.Value.GetValue());
		}

		{
			FScopeLock Lock(&ThreadSafeModValueCritSection);
			ThreadSafeModValueMap = MoveTemp(NewModulatorValues);
		}

		// Log stats
		SET_DWORD_STAT(STAT_AudioModulationBusCount, RefProxies.Buses.Num());
		SET_DWORD_STAT(STAT_AudioModulationMixCount, RefProxies.BusMixes.Num());
		SET_DWORD_STAT(STAT_AudioModulationGeneratorCount, RefProxies.Generators.Num());
		SET_DWORD_STAT(STAT_AudioModulationPatchCount, RefProxies.Patches.Num());
		SET_DWORD_STAT(STAT_AudioModulationProcQueueCount, CommandsProcessed);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		UpdateAudioInsightsTraceMessages(InElapsed);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

#if !UE_BUILD_SHIPPING
 		Debugger->UpdateDebugData(InElapsed, RefProxies, AudioDeviceId);
#endif // !UE_BUILD_SHIPPING
	}

	bool FAudioModulationSystem::IsControlBusMixActive(const USoundControlBusMix& InBusMix)
	{
		FBusMixId BusMixId = static_cast<FBusMixId>(InBusMix.GetUniqueID());		
		return ActiveBusMixIds.Contains(BusMixId);
	}

	void FAudioModulationSystem::SaveMixToProfile(const USoundControlBusMix& InBusMix, const int32 InProfileIndex)
	{
		check(IsInGameThread());

		RunCommandOnProcessingThread([this, MixToSerialize = TWeakObjectPtr<const USoundControlBusMix>(&InBusMix), InProfileIndex]()
		{
			if (!MixToSerialize.IsValid())
			{
				return;
			}

			const FBusMixId MixId = static_cast<FBusMixId>(MixToSerialize->GetUniqueID());
			const FString   MixName = MixToSerialize->GetName();

			FBusMixHandle MixHandle = FBusMixHandle::Get(MixId, RefProxies.BusMixes);
			const bool bIsActive = MixHandle.IsValid();
			if (!MixHandle.IsValid())
			{
				UE_LOGF(LogAudioModulation, Display, "Mix '%ls' is inactive, saving default object to profile '%i'.", *MixName, InProfileIndex);
				AsyncTask(ENamedThreads::GameThread, [this, MixToSerialize, InProfileIndex]()
				{
					AudioModulation::FProfileSerializer::Serialize(*MixToSerialize.Get(), InProfileIndex);
				});
				return;
			}

			UE_LOGF(LogAudioModulation, Display, "Mix '%ls' is active, saving current mix proxy state to profile '%i'.", *MixName, InProfileIndex);
			AudioModulation::FModulatorBusMixProxy& MixProxy = MixHandle.FindProxy();
			TMap<FBusId, FSoundModulationMixValue> PassedStageInfo;
			for (TPair<FBusId, FModulatorBusMixStageProxy>& Pair : MixProxy.Stages)
			{
				FModulatorBusMixStageProxy& Stage = Pair.Value;
				PassedStageInfo.Add(Pair.Key, Stage.Value);
			}

			AsyncTask(ENamedThreads::GameThread, [this, PassedStageInfo, MixToSerialize, InProfileIndex]()
			{
				if (!MixToSerialize.IsValid())
				{
					return;
				}
						
				TMap<FBusId, FSoundModulationMixValue> StageInfo = PassedStageInfo;
				USoundControlBusMix* TempMix = NewObject<USoundControlBusMix>(GetTransientPackage(), *FGuid::NewGuid().ToString(EGuidFormats::Short));

				// Buses on proxy may differ than those on uobject definition, so iterate and find by cached ids
				// and add to temp mix to be serialized.
				for (TObjectIterator<USoundControlBus> Itr; Itr; ++Itr)
				{
					if (USoundControlBus* Bus = *Itr)
					{
						FBusId ItrBusId = static_cast<FBusId>(Bus->GetUniqueID());
						if (FSoundModulationMixValue* Value = StageInfo.Find(ItrBusId))
						{
							FSoundControlBusMixStage BusMixStage;
							BusMixStage.Bus = Bus;
							BusMixStage.Value = *Value;
							TempMix->MixStages.Add(MoveTemp(BusMixStage));
						}
					}
				}

				const FString MixPath = MixToSerialize->GetPathName();
				AudioModulation::FProfileSerializer::Serialize(*TempMix, InProfileIndex, &MixPath);
			});
		});
	}

	TArray<FSoundControlBusMixStage> FAudioModulationSystem::LoadMixFromProfile(const int32 InProfileIndex, USoundControlBusMix& OutBusMix)
	{
		const FString TempName = FGuid::NewGuid().ToString(EGuidFormats::Short);
		if (USoundControlBusMix* TempMix = NewObject<USoundControlBusMix>(GetTransientPackage(), *TempName))
		{
			const FString MixPath = OutBusMix.GetPathName();
			AudioModulation::FProfileSerializer::Deserialize(InProfileIndex, *TempMix, &MixPath);
			UpdateMix(TempMix->MixStages, OutBusMix);
			return TempMix->MixStages;
		}

		return TArray<FSoundControlBusMixStage>();
	}

	void FAudioModulationSystem::RunCommandOnProcessingThread(TUniqueFunction<void()> Cmd)
	{
		if (IsInProcessingThread())
		{
			Cmd();
		}
		else
		{
			ProcessingThreadCommandQueue.Enqueue(MoveTemp(Cmd));
		}
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void FAudioModulationSystem::OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		// Ensures when a trace is started, our active trace messages are sent instantly without delay
		ModulatorActivePingTimeSec = 0.0f;
	}

	void FAudioModulationSystem::UpdateAudioInsightsTraceMessages(const double InElapsed)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bIsAudioModulationInsightsPluginEnabled || !bChannelEnabled)
		{
			return;
		}
		
		RunCommandOnProcessingThread([this, InElapsed]()
		{
			ModulatorActivePingTimeSec -= InElapsed;

			CurrentlyActiveModulatorIds.Reset();

			for (const auto& [GeneratorId, GeneratorProxy] : RefProxies.Generators)
			{
				TraceUpdateModulatorMessage<FModulatorGeneratorProxy>(GeneratorId, GeneratorProxy, EModulatorTraceType::Generator);
				CurrentlyActiveModulatorIds.Emplace(GeneratorId);
			}

			for (const auto& [PatchId, ParamaterPatchProxy] : RefProxies.Patches)
			{
				const int32 NumStages = ParamaterPatchProxy.InputProxies.Num();

				ContributingModulatorIds.Reset();
				ContributingModulatorValues.Reset();

				for (const FModulationInputProxy& Input : ParamaterPatchProxy.InputProxies)
				{
					ContributingModulatorIds.Emplace(Input.BusHandle.GetId());
					ContributingModulatorValues.Emplace(Input.ModStageValue);
				}

				TraceUpdateModulatorMessage<FModulationPatchRefProxy>(PatchId, ParamaterPatchProxy, EModulatorTraceType::ParameterPatch, ContributingModulatorIds, ContributingModulatorValues);
				CurrentlyActiveModulatorIds.Emplace(PatchId);
			}

			// For keeping track of the relationship between ControlBusMix (Id, value) and a ControlBus. Allowing ControlBuses to send through ControlBusMixes as contributors.
			ControlBusMixContributingDataContainer.Reset();

			for (const auto& [BusMixId, ModulatorBusMixProxy] : RefProxies.BusMixes)
			{
				const FModulatorBusMixProxy::EStatus CurrentStatus = ModulatorBusMixProxy.GetStatus();

				switch (CurrentStatus)
				{
					case FModulatorBusMixProxy::EStatus::Enabled:
					case FModulatorBusMixProxy::EStatus::Stopping:
					{
						const int32 NumStages = ModulatorBusMixProxy.Stages.Num();

						ContributingModulatorIds.Reset();
						ContributingModulatorValues.Reset();

						for (const auto& [BusId, BusMixStageProxy] : ModulatorBusMixProxy.Stages)
						{
							ContributingModulatorIds.Emplace(BusId);
							ContributingModulatorValues.Emplace(BusMixStageProxy.Value.GetCurrentValue());

							// Note: Intentionally do not empty this each BusMix iteration, as we require all data to be available for ControlBus contributing logic below
							ControlBusMixContributingDataContainer.Emplace(AudioInsightsControlBusMixData{BusMixId, BusId, BusMixStageProxy.Value.GetCurrentValue()});
						}

						TraceUpdateModulatorMessage<FModulatorBusMixProxy>(BusMixId, ModulatorBusMixProxy, EModulatorTraceType::ControlBusMix, ContributingModulatorIds, ContributingModulatorValues);
						CurrentlyActiveModulatorIds.Emplace(BusMixId);
					}
					break;
					
					default: break;
				}
			}

			for (const auto& [BusId, ControlBusProxy] : RefProxies.Buses)
			{
				ContributingModulatorIds.Reset();
				ContributingModulatorValues.Reset();

				for (const FGeneratorHandle& GeneratorHandle : ControlBusProxy.GetGeneratorHandles())
				{
					ContributingModulatorIds.Emplace(GeneratorHandle.GetId());
					ContributingModulatorValues.Emplace(GeneratorHandle.FindProxy().GetValue());
				}

				// Note: Multiple ControlBusMixes can contain the same ControlBus, therefore we must iterate over all ControlBusMixContributingDataContainer
				for (const AudioInsightsControlBusMixData& AudioInsightsControlBusMixData : ControlBusMixContributingDataContainer)
				{
					if (BusId == AudioInsightsControlBusMixData.ControlBusId)
					{
						ContributingModulatorIds.Emplace(AudioInsightsControlBusMixData.ControlBusMixId);
						ContributingModulatorValues.Emplace(AudioInsightsControlBusMixData.Value);
					}
				}

				TraceUpdateModulatorMessage<FControlBusProxy>(BusId, ControlBusProxy, EModulatorTraceType::ControlBus, ContributingModulatorIds, ContributingModulatorValues);
				CurrentlyActiveModulatorIds.Emplace(BusId);
			}

			for (const Audio::FModulatorId PreviouslyActiveModulatorId : PreviouslyActiveModulatorIds)
			{
				if (!CurrentlyActiveModulatorIds.Contains(PreviouslyActiveModulatorId))
				{
					TraceDeactivateModulatorMessage(PreviouslyActiveModulatorId);
				}
			}

			PreviouslyActiveModulatorIds = CurrentlyActiveModulatorIds;

			if (ModulatorActivePingTimeSec <= 0.0f)
			{
				ModulatorActivePingTimeSec = ModulatorActivePingIntervalTimeSec;
			}
		});
	}

	void FAudioModulationSystem::TraceActivateModulatorMessage(const Audio::FModulatorId InModulatorId,
		const EModulatorTraceType InModulatorTraceType,
		const FString& InModulatorName)
	{
		ensure(IsInProcessingThread());

		UE_TRACE_LOG(Audio, ActivateModulatorTraceMessage, AudioMixerChannel)
			<< ActivateModulatorTraceMessage.DeviceId(GetAudioDeviceId())
			<< ActivateModulatorTraceMessage.Timestamp(FPlatformTime::Cycles64())
			<< ActivateModulatorTraceMessage.ModulatorId(InModulatorId)
			<< ActivateModulatorTraceMessage.ModulatorType(static_cast<Audio::FModulatorTypeId>(InModulatorTraceType))
			<< ActivateModulatorTraceMessage.Name(*InModulatorName);
	}

	template <typename TModulatorProxy>
	void FAudioModulationSystem::TraceUpdateModulatorMessage(const Audio::FModulatorId InModulatorId,
		const TModulatorProxy& InModulatorProxy,
		const EModulatorTraceType InModulatorTraceType,
		TArrayView<const Audio::FModulatorId> InContributingModulatorIds /* = TArrayView<const Audio::FModulatorId>() */,
		TArrayView<const float> InContributingModulatorValues /* = TArrayView<const float>()*/)
	{
		ensure(IsInProcessingThread());
		ensure(InModulatorTraceType != EModulatorTraceType::COUNT);

		if (ModulatorActivePingTimeSec <= 0.0f)
		{
			TraceActivateModulatorMessage(InModulatorId, InModulatorTraceType, *(InModulatorProxy.GetPath().ToString()));
		}

		// Note: ControlBusMixes do not output a single value, nor can they be bypassed. Therefore using a compile time conditional to allow for flexible message data.
		if constexpr (std::is_same_v<TModulatorProxy, FModulatorBusMixProxy>)
		{
			UE_TRACE_LOG(Audio, UpdateModulatorTraceMessage, AudioMixerChannel)
			<< UpdateModulatorTraceMessage.DeviceId(GetAudioDeviceId())
			<< UpdateModulatorTraceMessage.Timestamp(FPlatformTime::Cycles64())
			<< UpdateModulatorTraceMessage.ModulatorId(InModulatorId)
			<< UpdateModulatorTraceMessage.ModulatorValue(0) // Note: BusMixes do not output a single value
			<< UpdateModulatorTraceMessage.IsBypassed(false) // Note: BusMixes can not be bypassed
			<< UpdateModulatorTraceMessage.ContributingModulatorIds(InContributingModulatorIds.GetData(), InContributingModulatorIds.Num())
			<< UpdateModulatorTraceMessage.ContributingModulatorValues(InContributingModulatorValues.GetData(), InContributingModulatorValues.Num());
		}
		else
		{
			UE_TRACE_LOG(Audio, UpdateModulatorTraceMessage, AudioMixerChannel)
			<< UpdateModulatorTraceMessage.DeviceId(GetAudioDeviceId())
			<< UpdateModulatorTraceMessage.Timestamp(FPlatformTime::Cycles64())
			<< UpdateModulatorTraceMessage.ModulatorId(InModulatorId)
			<< UpdateModulatorTraceMessage.ModulatorValue(InModulatorProxy.GetValue())
			<< UpdateModulatorTraceMessage.IsBypassed(InModulatorProxy.IsBypassed())
			<< UpdateModulatorTraceMessage.ContributingModulatorIds(InContributingModulatorIds.GetData(), InContributingModulatorIds.Num())
			<< UpdateModulatorTraceMessage.ContributingModulatorValues(InContributingModulatorValues.GetData(), InContributingModulatorValues.Num());
		}

		if (!PreviouslyActiveModulatorIds.Contains(InModulatorId))
		{
			const FString AssetPath = InModulatorProxy.GetPath().ToString();
			ActiveModulatorPaths.Emplace(InModulatorId, AssetPath);
			Audio::Trace::EventLog::SendEvent(GetAudioDeviceId(), Audio::Modulation::Trace::EventLog::ID::ModulatorActivated, InModulatorId, AssetPath);
		}
	}

	void FAudioModulationSystem::TraceDeactivateModulatorMessage(const Audio::FModulatorId InModulatorId)
	{
		ensure(IsInProcessingThread());

		UE_TRACE_LOG(Audio, DeactivateModulatorTraceMessage, AudioMixerChannel)
			<< DeactivateModulatorTraceMessage.DeviceId(GetAudioDeviceId())
			<< DeactivateModulatorTraceMessage.Timestamp(FPlatformTime::Cycles64())
			<< DeactivateModulatorTraceMessage.ModulatorId(InModulatorId);

		FString RemovedPath;
		ActiveModulatorPaths.RemoveAndCopyValue(InModulatorId, RemovedPath);
		Audio::Trace::EventLog::SendEvent(GetAudioDeviceId(), Audio::Modulation::Trace::EventLog::ID::ModulatorDeactivated, InModulatorId, RemovedPath);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	Audio::FModulatorTypeId FAudioModulationSystem::RegisterModulator(Audio::FModulatorHandleId InHandleId, const FControlBusSettings& InSettings)
	{
		FControlBusSettings CachedSettings = InSettings;
		RegisterModulator(InHandleId, MoveTemp(CachedSettings), RefProxies.Buses, RefModulators.BusMap);

		return static_cast<Audio::FModulatorTypeId>(EModulatorType::Bus);
	}

	Audio::FModulatorTypeId FAudioModulationSystem::RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationGeneratorSettings& InSettings)
	{
		FModulationGeneratorSettings CachedSettings = InSettings;
		RegisterModulator(InHandleId, MoveTemp(CachedSettings), RefProxies.Generators, RefModulators.GeneratorMap, [this](FGeneratorHandle& NewHandle)
		{
			NewHandle.FindProxy().Init(AudioDeviceId);
		});
		return static_cast<Audio::FModulatorTypeId>(EModulatorType::Generator);
	}

	Audio::FModulatorTypeId FAudioModulationSystem::RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationPatchSettings& InSettings)
	{
		FModulationPatchSettings CachedSettings = InSettings;
		RegisterModulator(InHandleId, MoveTemp(CachedSettings), RefProxies.Patches, RefModulators.PatchMap);

		return static_cast<Audio::FModulatorTypeId>(EModulatorType::Patch);
	}

	void FAudioModulationSystem::RegisterModulator(Audio::FModulatorHandleId InHandleId, Audio::FModulatorId InModulatorId)
	{
		RunCommandOnProcessingThread([this, InHandleId, InModulatorId]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAudioModulationSystem::RegisterModulator);
			FPatchHandle PatchHandle = FPatchHandle::Get(static_cast<FPatchId>(InModulatorId), RefProxies.Patches);
			if (PatchHandle.IsValid())
			{
				if (FModulatorHandleSet* HandleSet = RefModulators.PatchMap.Find(PatchHandle))
				{
					HandleSet->Add(InHandleId);
				}
				return;
			}

			FBusHandle BusHandle = FBusHandle::Get(static_cast<FBusId>(InModulatorId), RefProxies.Buses);
			if (BusHandle.IsValid())
			{
				if (FModulatorHandleSet* HandleSet = RefModulators.BusMap.Find(BusHandle))
				{
					HandleSet->Add(InHandleId);
				}
				return;
			}

			FGeneratorHandle GeneratorHandle = FGeneratorHandle::Get(static_cast<FGeneratorId>(InModulatorId), RefProxies.Generators);
			if (GeneratorHandle.IsValid())
			{
				if (FModulatorHandleSet* HandleSet = RefModulators.GeneratorMap.Find(GeneratorHandle))
				{
					HandleSet->Add(InHandleId);
				}
				return;
			}

			ensureAlwaysMsgf(false, TEXT("Failed to register modulator handle '%i' with pre-existing modulator '%i'"), static_cast<int32>(InHandleId), static_cast<int32>(InModulatorId));
		});
	}

	void FAudioModulationSystem::SoloBusMix(const USoundControlBusMix& InBusMix)
	{
		RunCommandOnProcessingThread([this, BusMixSettings = FModulatorBusMixSettings(InBusMix)]() mutable
		{
			bool bMixActive = false;
			for (TPair<FBusMixId, FModulatorBusMixProxy>& Pair : RefProxies.BusMixes)
			{
				if (Pair.Key == BusMixSettings.GetId())
				{
					bMixActive = true;
				}
				else
				{
					Pair.Value.SetStopping();
				}
			}

			if (!bMixActive)
			{
				ActivateBusMix(MoveTemp(BusMixSettings));
			}
		});
	}

	void FAudioModulationSystem::UnregisterModulator(const Audio::FModulatorHandle& InHandle)
	{
		RunCommandOnProcessingThread([this, ModId = InHandle.GetModulatorId(), HandleId = InHandle.GetHandleId()]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAudioModulationSystem::UnregisterModulator);

			FPatchHandle PatchHandle = FPatchHandle::Get(static_cast<FPatchId>(ModId), RefProxies.Patches);
			if (UnregisterModulator<FPatchHandle>(PatchHandle, RefModulators.PatchMap, HandleId))
			{
				return;
			}

			FBusHandle BusHandle = FBusHandle::Get(static_cast<FBusId>(ModId), RefProxies.Buses);
			if (UnregisterModulator<FBusHandle>(BusHandle, RefModulators.BusMap, HandleId))
			{
#if UE_AUDIO_PROFILERTRACE_ENABLED
				if (RefModulators.BusMap.Find(BusHandle) == nullptr)
				{
					FControlBusProxy& ControlBusProxy = BusHandle.FindProxy();
				}
#endif

				return;
			}

			FGeneratorHandle GeneratorHandle = FGeneratorHandle::Get(static_cast<FGeneratorId>(ModId), RefProxies.Generators);
			if (UnregisterModulator<FGeneratorHandle>(GeneratorHandle, RefModulators.GeneratorMap, HandleId))
			{
				return;
			}
		});
	}

	void FAudioModulationSystem::UpdateMix(const TArray<FSoundControlBusMixStage>& InStages, USoundControlBusMix& InOutMix, bool bInUpdateObject, float InFadeTime, double Duration, bool bRetriggerOnActivation)
	{
		if (bInUpdateObject)
		{
			TMap<uint32, const FSoundControlBusMixStage*> UpdatedStageBuses;
			for (const FSoundControlBusMixStage& Stage : InStages)
			{
				if (Stage.Bus)
				{
					UpdatedStageBuses.Add(Stage.Bus->GetUniqueID(), &Stage);
				}
			}

			bool bMarkDirty = false;
			for (FSoundControlBusMixStage& Stage : InOutMix.MixStages)
			{
				if (!Stage.Bus)
				{
					continue;
				}

				if (const FSoundControlBusMixStage* BusStage = UpdatedStageBuses.FindRef(Stage.Bus->GetUniqueID()))
				{
					Stage = *BusStage;
					bMarkDirty = true;
				}
			}
			InOutMix.MarkPackageDirty();
		}

		const FBusMixId MixId = static_cast<FBusMixId>(InOutMix.GetUniqueID());

		TArray<FModulatorBusMixStageSettings> StageSettings;
		for (const FSoundControlBusMixStage& Stage : InStages)
		{
			if (Stage.Bus)
			{
				StageSettings.Emplace(Stage);
			}
		}
	
		const FString BusMixName = InOutMix.GetName();
		RunCommandOnProcessingThread([this, MixId, StageSettings, InFadeTime, BusMixName, Duration, bRetriggerOnActivation]()
		{
			if (FModulatorBusMixProxy* BusMixes = RefProxies.BusMixes.Find(MixId))
			{
				BusMixes->SetMixData(StageSettings, InFadeTime, BusMixName, Duration, bRetriggerOnActivation);
			}
			else
			{
				UE_LOGF(LogAudioModulation, Display, "Could not update mix '%ls' because the mix is not currently active.", *BusMixName);
			}
		});
	}

	void FAudioModulationSystem::UpdateMixByFilter(
		const FString& InAddressFilter,
		const TSubclassOf<USoundModulationParameter>& InParamClassFilter,
		USoundModulationParameter* InParamFilter,
		float InValue,
		float InFadeTime,
		USoundControlBusMix& InOutMix,
		bool bInUpdateObject)
	{
		const uint32 ParamClassId = InParamClassFilter ? InParamClassFilter->GetUniqueID() : INDEX_NONE;
		const uint32 ParamId = InParamFilter ? InParamFilter->GetUniqueID() : INDEX_NONE;

		if (bInUpdateObject)
		{
			bool bMarkDirty = false;
			for (FSoundControlBusMixStage& Stage : InOutMix.MixStages)
			{
				if (!Stage.Bus)
				{
					continue;
				}

				if (USoundModulationParameter* Parameter = Stage.Bus->Parameter)
				{
					if (ParamId != INDEX_NONE && ParamId != Parameter->GetUniqueID())
					{
						continue;
					}

					if (UClass* Class = Parameter->GetClass())
					{
						if (ParamClassId != INDEX_NONE && ParamClassId != Class->GetUniqueID())
						{
							continue;
						}
					}
				}

				if (!FAudioAddressPattern::PartsMatch(InAddressFilter, Stage.Bus->Address))
				{
					continue;
				}

				Stage.Value.TargetValue = InValue;
				Stage.Value.SetActiveFade(FSoundModulationMixValue::EActiveFade::Override, InFadeTime);
				bMarkDirty = true;
			}

			if (bMarkDirty)
			{
				InOutMix.MarkPackageDirty();
			}
		}

		const FString	AddressFilter = InAddressFilter;
		const FBusMixId MixId = static_cast<FBusMixId>(InOutMix.GetUniqueID());
		RunCommandOnProcessingThread([this, ParamClassId, ParamId, MixId, AddressFilter, InValue, InFadeTime]()
		{
			if (FModulatorBusMixProxy* MixProxy = RefProxies.BusMixes.Find(MixId))
			{
				MixProxy->SetMixByFilter(AddressFilter, ParamClassId, ParamId, InValue, InFadeTime);
			}
		});
	}

	void FAudioModulationSystem::UpdateMix(const USoundControlBusMix& InMix, float InFadeTime)
	{
		RunCommandOnProcessingThread([this, MixSettings = FModulatorBusMixSettings(InMix), InFadeTime]() mutable
		{
			FBusMixHandle BusMixHandle = FBusMixHandle::Get(MixSettings.GetId(), RefProxies.BusMixes);
			if (BusMixHandle.IsValid())
			{
				FModulatorBusMixProxy& MixProxy = BusMixHandle.FindProxy();
				if (MixProxy.GetStatus() == FModulatorBusMixProxy::EStatus::Enabled)
				{
					MixProxy = MoveTemp(MixSettings);
					for (TPair<FBusId, FModulatorBusMixStageProxy>& Stage : MixProxy.Stages)
					{
						Stage.Value.Value.SetActiveFade(FSoundModulationMixValue::EActiveFade::Override, InFadeTime);
					}
				}
			}
#if !UE_BUILD_SHIPPING
			else
			{
				UE_LOGF(LogAudioModulation, Verbose, "Update to '%ls' Ignored: Control Bus Mix is inactive.", *MixSettings.GetName().ToString());
			}
#endif // !UE_BUILD_SHIPPING
		});
	}

	void FAudioModulationSystem::UpdateModulator(const USoundModulatorBase& InModulator)
	{
		if (const USoundModulationGenerator* InGenerator = Cast<USoundModulationGenerator>(&InModulator))
		{
			RunCommandOnProcessingThread([this, GeneratorSettings = FModulationGeneratorSettings(*InGenerator)]() mutable
			{
				FGeneratorHandle GeneratorHandle = FGeneratorHandle::Get(GeneratorSettings.GetId(), RefProxies.Generators);
				if (GeneratorHandle.IsValid())
				{
					GeneratorHandle.FindProxy() = MoveTemp(GeneratorSettings);
				}
#if !UE_BUILD_SHIPPING
				else
				{
					UE_LOGF(LogAudioModulation, Verbose, "Update to '%ls' Ignored: Generator is inactive.", *GeneratorSettings.GetName().ToString());
				}
#endif // !UE_BUILD_SHIPPING
			});
		}

		if (const USoundControlBus* InBus = Cast<USoundControlBus>(&InModulator))
		{
			RunCommandOnProcessingThread([this, BusSettings = FControlBusSettings(*InBus)]() mutable
			{
				FBusHandle BusHandle = FBusHandle::Get(BusSettings.GetId(), RefProxies.Buses);
				if (BusHandle.IsValid())
				{
					FControlBusProxy& BusProxy = BusHandle.FindProxy();
					BusProxy = MoveTemp(BusSettings);
				}
#if !UE_BUILD_SHIPPING
				else
				{
					UE_LOGF(LogAudioModulation, Verbose, "Update to '%ls' Ignored: Control Bus is inactive.", *BusSettings.GetName().ToString());
				}
#endif // !UE_BUILD_SHIPPING
			});
		}

		if (const USoundModulationPatch* InPatch = Cast<USoundModulationPatch>(&InModulator))
		{
			RunCommandOnProcessingThread([this, PatchSettings = FModulationPatchSettings(*InPatch)]() mutable
			{
				FPatchHandle PatchHandle = FPatchHandle::Get(PatchSettings.GetId(), RefProxies.Patches);
				if (PatchHandle.IsValid())
				{
					FModulationPatchRefProxy& PatchProxy = PatchHandle.FindProxy();
					PatchProxy = MoveTemp(PatchSettings);
				}
#if !UE_BUILD_SHIPPING
				else
				{
					UE_LOGF(LogAudioModulation, Verbose, "Update to '%ls' Ignored: Patch is inactive.", *PatchSettings.GetName().ToString());
				}
#endif // !UE_BUILD_SHIPPING
			});
		}
	}
} // namespace AudioModulation
#endif // WITH_AUDIOMODULATION
