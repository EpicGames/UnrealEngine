// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioModulation.h"
#include "Containers/Queue.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "IAudioModulation.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "SoundControlBusMixProxy.h"
#include "SoundControlBusProxy.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatchProxy.h"
#include "SoundModulationProxy.h"
#include "SoundModulationValue.h"
#include "SoundModulationGenerator.h"
#include "SoundModulationGeneratorProxy.h"
#include "Templates/Atomic.h"
#include "Templates/Function.h"
#include "UObject/GCObject.h"

#if WITH_AUDIOMODULATION
#if !UE_BUILD_SHIPPING
#include "AudioModulationDebugger.h"
#endif // !UE_BUILD_SHIPPING
#endif // WITH_AUDIOMODULATION

namespace AudioModulation
{
	// Forward Declarations
	struct FControlBusSettings;
	struct FModulationGeneratorSettings;
	struct FModulationPatchSettings;

	class FControlBusProxy;
	class FModulationInputProxy;
	class FModulationPatchProxy;
	class FModulationPatchRefProxy;
	class FModulatorBusMixStageProxy;

	struct FReferencedProxies
	{
		FBusMixProxyMap BusMixes;
		FBusProxyMap Buses;
		FGeneratorProxyMap Generators;
		FPatchProxyMap Patches;
	};

	using FModulatorHandleSet = TSet<Audio::FModulatorHandleId>;

	struct FReferencedModulators
	{
		TMap<FPatchHandle, FModulatorHandleSet> PatchMap;
		TMap<FBusHandle, FModulatorHandleSet> BusMap;
		TMap<FGeneratorHandle, FModulatorHandleSet> GeneratorMap;
	};
} // namespace AudioModulation

#if WITH_AUDIOMODULATION
namespace AudioModulation
{
	class FAudioModulationSystem : public FGCObject
	{
	public:
		~FAudioModulationSystem();

		void Initialize(const FAudioPluginInitializationParams& InitializationParams);

		UE_DEPRECATED(5.4, "Activation of modulators in this manner is now deprecated. Use USoundModulationDestinations to safely activate and track a given modulator")
		void ActivateBus(const USoundControlBus& InBus);

		void ActivateBusMix(FModulatorBusMixSettings&& InSettings);
		void ActivateBusMix(const USoundControlBusMix& InBusMix);

		UE_DEPRECATED(5.4, "Activation of modulators in this manner is now deprecated. Use USoundModulationDestinations to safely activate and track a given modulator")
		void ActivateGenerator(const USoundModulationGenerator& InGenerator);

		// FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		// End of FGCObject interface

		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestinations to safely activate and track a given modulator")
		void DeactivateBus(const USoundControlBus& InBus);

		/** Deactivates given bus mix */
		void DeactivateBusMix(const USoundControlBusMix& InBusMix);
		void DeactivateBusMixByID(const uint32 ID);

		/** Deactivates all bus mixes */
		void DeactivateAllBusMixes();

		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestinations to safely activate and track a given modulator")
		void DeactivateGenerator(const USoundModulationGenerator& InGenerator);

		void ProcessModulators(const double InElapsed);
		void SoloBusMix(const USoundControlBusMix& InBusMix);

		Audio::FDeviceId GetAudioDeviceId() const;

		/* Register new handle with given a given modulator that may or may not already be active (i.e. registered).
		 * If already registered, depending on modulator type, may or may not refresh proxy based on provided settings.
		 */
		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FControlBusSettings& InSettings);
		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationGeneratorSettings& InSettings);
		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationPatchSettings& InSettings);

		/* Register new handle with given Id with a modulator that is already active (i.e. registered). Used primarily for copying modulation handles. */
		void RegisterModulator(Audio::FModulatorHandleId InHandleId, Audio::FModulatorId InModulatorId);

		/* Attempts to get the modulator value from the AudioRender Thread. */
		bool GetModulatorValue(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const;

		/* Attempts to get the modulator value from any thread (lock contentious).*/
		bool GetModulatorValueThreadSafe(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const;

		/* Attempts to get the modulator value from any thread (lock contentious).*/
		bool GetModulatorValueThreadSafe(uint32 ModulatorID, float& OutValue) const;

		void UnregisterModulator(const Audio::FModulatorHandle& InHandle);

		bool IsControlBusMixActive(const USoundControlBusMix& InBusMix);

		/* Saves mix to .ini profile for fast iterative development that does not require re-cooking a mix */
		void SaveMixToProfile(const USoundControlBusMix& InBusMix, const int32 InProfileIndex);

		/* Loads mix from .ini profile for iterative development that does not require re-cooking a mix. Returns copy
		 * of mix stage values saved in profile. */
		TArray<FSoundControlBusMixStage> LoadMixFromProfile(const int32 InProfileIndex, USoundControlBusMix& OutBusMix);

		/*
		 * Updates mix/mix by filter, modifying the mix instance if it is active. If bInUpdateObject is true,
		 * updates UObject definition in addition to proxy.
		 */
		void UpdateMix(const TArray<FSoundControlBusMixStage>& InStages, USoundControlBusMix& InOutMix, bool bInUpdateObject = false, float InFadeTime = -1.0f, double Duration = -1.0, bool bRetriggerOnActivation = false);
		void UpdateMixByFilter(const FString& InAddressFilter, const TSubclassOf<USoundModulationParameter>& InParamClassFilter, USoundModulationParameter* InParamFilter, float Value, float FadeTime, USoundControlBusMix& InOutMix, bool bInUpdateObject = false);

		/*
		 * Commits any changes from a mix applied to a UObject definition to mix instance if active.
		 */
		void UpdateMix(const USoundControlBusMix& InMix, float InFadeTime = -1.0f);

		/* Sets the global bus mix value if over the prescribed time. If FadeTime is non-positive, applies the value immediately. */
		void SetGlobalBusMixValue(USoundControlBus& Bus, float Value, float FadeTime = -1.0f);

		/* Clears the global bus mix value over the prescribed FadeTime. If FadeTime is non-positive, returns to the bus's respective parameter default immediately. */
		void ClearGlobalBusMixValue(const USoundControlBus& InBus, float FadeTime = -1.0f);

		/* Clears all global bus mix values over the prescribed FadeTime. If FadeTime is non-positive, returns to the bus's respective parameter default immediately. */
		void ClearAllGlobalBusMixValues(float FadeTime = -1.0f);

		/* Returns whether the given bus Mix is a Global Bus Mix, created using the SetGlobalBusMixValue function. */
		bool IsBusMixGlobal(FBusMixId InBusMixId) const;
		
		/* Create a mix from an array of buses where a supplied default value and associated timing parameters are applied for each bus's stage.
		 * InOuter, when supplied, is used as the new mix's outer object so its lifetime tracks the caller's scope (typically the supplied world);
		 * the transient package is used otherwise.
		 */
		USoundControlBusMix* CreateBusMixFromValue(FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime = -1.0f, float ReleaseTime = -1.0f, UObject* InOuter = nullptr);

		/*
		 * Commits any changes from a modulator type applied to a UObject definition
		 * to modulator instance if active (i.e. Control Bus, Control Bus Modulator)
		 */
		void UpdateModulator(const USoundModulatorBase& InModulator);

		void OnAuditionEnd();

	private:
		/* Calculates modulation value, storing it in the provided float reference and returns if value changed */
		bool CalculateModulationValue(FModulationPatchProxy& OutProxy, float& OutValue) const;

		/* Whether or not caller is in processing thread or not */
		bool IsInProcessingThread() const;

		/* Runs the provided command on the audio render thread (at the beginning of the ProcessModulators call) */
		void RunCommandOnProcessingThread(TUniqueFunction<void()> Cmd);

		/* Template for register calls that move the modulator settings objects onto the Audio Processing Thread & create proxies.
		 * If the proxy is already found, adds provided HandleId as reference to given proxy. Does *not* update the proxy with the
		 * given settings.  If update is desired on an existing proxy, UpdateModulator must be explicitly called.
		 * @tparam THandleType Type of modulator handle used to access the proxy on the Processing Thread
		 * @tparam TModSettings Modulation settings to move and use if proxy construction is required on the Audio Processing Thread
		 * @tparam TMapType MapType used to cache the corresponding proxy id & proxy
		 * @tparam TInitFunction (Optional) Function type used to call on the AudioProcessingThread immediately following proxy construction
		 * @param InHandleId HandleId associated with the proxy to be retrieved/generated.
		 * @param InModSettings ModulatorSettings to be used if construction of proxy is required on Audio Processing Thread
		 * @param OutProxyMap Map to find or add proxy to if constructed
		 * @param InInitFunction Function type used to call on the AudioProcessingThread immediately following proxy generation
		 */
		template <typename THandleType, typename TModSettings, typename TMapType, typename TInitFunction = TUniqueFunction<void(THandleType)>>
		void RegisterModulator(Audio::FModulatorHandleId InHandleId, TModSettings&& InModSettings, TMapType& OutProxyMap, TMap<THandleType, FModulatorHandleSet>& OutModMap, TInitFunction InInitFunction = [](THandleType){ })
		{
			check(InHandleId != INDEX_NONE);

			RunCommandOnProcessingThread([
				this,
				InHandleId,
				ModSettings = MoveTemp(InModSettings),
				InitFunction = MoveTemp(InInitFunction),
				PassedProxyMap = &OutProxyMap,
				PassedModMap = &OutModMap
			]() mutable
			{
				check(PassedProxyMap);
				check(PassedModMap);

				THandleType Handle = THandleType::Create(MoveTemp(ModSettings), *PassedProxyMap, *this);
				PassedModMap->FindOrAdd(Handle).Add(InHandleId);

				InitFunction(Handle);
			});
		}

		template <typename THandleType>
		bool UnregisterModulator(THandleType InModHandle, TMap<THandleType, FModulatorHandleSet>& OutHandleMap, const Audio::FModulatorHandleId InHandleId)
		{
			bool bHandleRemoved = false;

			if (!InModHandle.IsValid())
			{
				return bHandleRemoved;
			}

			if (FModulatorHandleSet* HandleSet = OutHandleMap.Find(InModHandle))
			{
				bHandleRemoved = HandleSet->Remove(InHandleId) > 0;
				if (HandleSet->IsEmpty())
				{
					OutHandleMap.Remove(InModHandle);
				}
			}

			return bHandleRemoved;
		}

		FReferencedProxies RefProxies;

		// Critical section & map of copied, computed modulation values for
		// use from the decoder threads & respective MetaSound getter nodes.
		mutable FCriticalSection ThreadSafeModValueCritSection;
		TMap<Audio::FModulatorId, float> ThreadSafeModValueMap;

		TSet<FBusMixId> ActiveBusMixIds;

		TSet<FBusHandle> ManuallyActivatedBuses;
		TSet<FBusMixHandle> ManuallyActivatedBusMixes;
		TSet<FGeneratorHandle> ManuallyActivatedGenerators;

		// Global mixes each containing a single stage for any globally manipulated bus stage value.
		// the key in this map is the Bus's unique ID, not the Control Bus Mix's.
		TMap<uint32, TObjectPtr<USoundControlBusMix>> ActiveGlobalBusValueMixes;

		// Command queue to be consumed on processing thread 
		TQueue<TUniqueFunction<void()>, EQueueMode::Mpsc> ProcessingThreadCommandQueue;

		// Thread modulators are processed on
		TAtomic<uint32> ProcessingThreadId { 0 };

		// Collection of maps with modulator handles to referencing object ids used by externally managing objects
		FReferencedModulators RefModulators;

		Audio::FDeviceId AudioDeviceId = INDEX_NONE;

#if UE_AUDIO_PROFILERTRACE_ENABLED

		/*
		* Note: The following mirrors data from Engine/Plugins/Runtime/AudioModulationInsights
		*
		* Duplication is required as AudioInsights cannot depend on any engine code
		* As it must be able to run in standalone (where engine code is not available)
		* 
		* Caution: Since this means we no longer have a single source of truth
		* We must ensure this data is kept in sync
		*
		* Intentionally not using the EModulatorType within AudioModulationSystem.cpp:
		* 1. For the reasons above
		* 2. To allow inclusion of Control Bus Mixes
		* 3. To avoid being unintentionally stomped over by future engine changes to EModulatorType
		*/

		// Mirrors EModulatorTraceType: Engine/Plugins/Runtime/AudioModulationInsights/Source/AudioModulationInsights/Private/AudioModulationDataTypes.h
		enum class EModulatorTraceType : Audio::FModulatorTypeId
		{
			ControlBus,
			ControlBusMix,
			Generator,
			ParameterPatch,

			COUNT
		};

		void OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

		void UpdateAudioInsightsTraceMessages(const double InElapsed);

		void TraceActivateModulatorMessage(const Audio::FModulatorId InModulatorId,
			const EModulatorTraceType InModulatorTraceType,
			const FString& InModulatorName);

		template <typename TModulatorProxy>
		void TraceUpdateModulatorMessage(const Audio::FModulatorId InModulatorId,
			const TModulatorProxy& InModulatorProxy,
			const EModulatorTraceType InModulatorTraceType,
			TArrayView<const Audio::FModulatorId> InContributingModulatorIds = TArrayView<const Audio::FModulatorId>(),
			TArrayView<const float> InContributingModulatorValues = TArrayView<const float>());

		void TraceDeactivateModulatorMessage(const Audio::FModulatorId InModulatorId);

		bool bIsAudioModulationInsightsPluginEnabled { false };

		float ModulatorActivePingTimeSec { 0.0f };
		static constexpr float ModulatorActivePingIntervalTimeSec { 1.0f };

		struct AudioInsightsControlBusMixData
		{
			Audio::FModulatorId ControlBusMixId;
			Audio::FModulatorId ControlBusId;
			float Value;
		};

		TArray<Audio::FModulatorId, TInlineAllocator<32>> CurrentlyActiveModulatorIds;
		TArray<Audio::FModulatorId, TInlineAllocator<32>> PreviouslyActiveModulatorIds;
		TMap<Audio::FModulatorId, FString> ActiveModulatorPaths;
		
		TArray<float, TInlineAllocator<32>> ContributingModulatorValues;
		TArray<Audio::FModulatorId, TInlineAllocator<32>> ContributingModulatorIds;
		TArray<AudioInsightsControlBusMixData, TInlineAllocator<32>> ControlBusMixContributingDataContainer;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

#if !UE_BUILD_SHIPPING
	public:
		void SetDebugBusFilter(const FString* InFilter);
		void SetDebugMixFilter(const FString* InFilter);
		void SetDebugMatrixEnabled(bool bInIsEnabled);
		void SetDebugGeneratorsEnabled(bool bInIsEnabled);
		void SetDebugGeneratorFilter(const FString* InFilter);
		void SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled);
		virtual void SetDebugActiveMixesEnabled(bool bInIsEnabled);
		virtual void SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled);
		bool OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		int OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation);
		bool OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream);

	private:
		TSharedPtr<FAudioModulationDebugger> Debugger;
#endif // !UE_BUILD_SHIPPING

		friend FControlBusProxy;
		friend FModulationInputProxy;
		friend FModulationPatchProxy;
		friend FModulationPatchRefProxy;
		friend FModulatorBusMixStageProxy;
	};
} // namespace AudioModulation

#else // !WITH_AUDIOMODULATION

namespace AudioModulation
{
	// Null implementation for compiler
	class FAudioModulationSystem
	{
	public:
		~FAudioModulationSystem();

		void Initialize(const FAudioPluginInitializationParams& InitializationParams) { }

		void OnAuditionEnd() { }

#if !UE_BUILD_SHIPPING
		void SetDebugBusFilter(const FString* InFilter) { }
		void SetDebugMixFilter(const FString* InFilter) { }
		void SetDebugMatrixEnabled(bool bInIsEnabled) { }
		void SetDebugGeneratorsEnabled(bool bInIsEnabled) { }
		void SetDebugGeneratorFilter(const FString* InFilter) { }
		void SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled) { }
		bool OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream) { return false; }
		int OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation) { return Y; }
		bool OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream) { return false; }
#endif // !UE_BUILD_SHIPPING

		void ActivateBus(const USoundControlBus& InBus) { }
		void ActivateBusMix(FModulatorBusMixSettings&& InSettings) { }
		void ActivateBusMix(const USoundControlBusMix& InBusMix) { }
		void ActivateGenerator(const USoundModulationGenerator& InGenerator) { }

		void DeactivateAllBusMixes() { }
		void DeactivateBus(const USoundControlBus& InBus) { }
		void DeactivateBusMix(const USoundControlBusMix& InBusMix) { }
		void DeactivateBusMixByID(const uint32 ID) {}
		void DeactivateGenerator(const USoundModulationGenerator& InGenerator) { }

		Audio::FDeviceId GetAudioDeviceId() const { return 0; }

		bool IsControlBusMixActive(const USoundControlBusMix& InBusMix) { return false; }

		void SaveMixToProfile(const USoundControlBusMix& InBusMix, const int32 InProfileIndex) { }
		TArray<FSoundControlBusMixStage> LoadMixFromProfile(const int32 InProfileIndex, USoundControlBusMix& OutBusMix) { return { }; }

		void SetGlobalBusMixValue(USoundControlBus& Bus, float Value, float FadeTime) { }
		void ClearGlobalBusMixValue(const USoundControlBus& InBus, float FadeTime) { }
		void ClearAllGlobalBusMixValues(float FadeTime) { }

		USoundControlBusMix* CreateBusMixFromValue(FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime = -1.0f, float ReleaseTime = -1.0f, UObject* InOuter = nullptr) { return nullptr; }

		void ProcessModulators(const double InElapsed) { }
		void SoloBusMix(const USoundControlBusMix& InBusMix) { }

		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FControlBusSettings& InSettings) { return 0; }
		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationGeneratorSettings& InSettings) { return 0; }
		Audio::FModulatorTypeId RegisterModulator(Audio::FModulatorHandleId InHandleId, const FModulationPatchSettings& InSettings) { return 0; }

		void RegisterModulator(Audio::FModulatorHandleId InHandleId, Audio::FModulatorId InModulatorId) { }
		bool GetModulatorValue(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const { return false; }
		bool GetModulatorValueThreadSafe(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const { return false; }
		bool GetModulatorValueThreadSafe(uint32 ModulatorID, float& OutValue) const { return false; }
		void UnregisterModulator(const Audio::FModulatorHandle& InHandle) { }

		inline bool FAudioModulationSystem::IsControlBusMixActive(const USoundControlBusMix* InBusMix)
		{
			return false;
		}

		void UpdateMix(const USoundControlBusMix& InMix, float InFadeTime = -1.0f) { }
		void UpdateMix(const TArray<FSoundControlBusMixStage>& InStages, USoundControlBusMix& InOutMix, bool bUpdateObject = false, float InFadeTime = -1.0f) { }
		void UpdateMixByFilter(const FString& InAddressFilter, const TSubclassOf<USoundModulationParameter>& InParamClassFilter, USoundModulationParameter* InParamFilter, float InValue, float InFadeTime, USoundControlBusMix& InOutMix, bool bUpdateObject = false) { }
		void UpdateModulator(const USoundModulatorBase& InModulator) { }

		private:
			FReferencedProxies RefProxies;

			friend FControlBusProxy;
			friend FModulationInputProxy;
			friend FModulationPatchProxy;
			friend FModulationPatchRefProxy;
			friend FModulatorBusMixStageProxy;
	};
} // namespace AudioModulation
#endif // WITH_AUDIOMODULATION
