// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationTrace.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "AudioInsightsEventLog"

namespace Audio::Modulation::Trace::EventLog::ID
{
	const FString ModulatorActivated   = LOCTEXT("EventLogTraceMessage_ModulatorActivated",   "Modulator Activated").ToString();
	const FString ModulatorDeactivated = LOCTEXT("EventLogTraceMessage_ModulatorDeactivated", "Modulator Deactivated").ToString();
} // namespace Audio::Modulation::Trace::EventLog::ID

#undef LOCTEXT_NAMESPACE
