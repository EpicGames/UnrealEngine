// Copyright Epic Games, Inc. All Rights Reserved.
#include "Generators/SoundModulationADEnvelope.h"

#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationSystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationADEnvelope)


namespace AudioModulation
{
	namespace GeneratorADEnvelopePrivate
	{
		static const TArray<FString> DebugCategories =
		{
			TEXT("Value"),
			TEXT("Attack Time"),
			TEXT("Decay Time"),
			TEXT("Attack Curve"),
			TEXT("Decay Curve"),
			TEXT("Looping")
		};

		static const FString DebugName = TEXT("AD Envelope");
	} // GeneratorADEnvelopePrivate

	class FADEnvelopeGenerator : public IGenerator
	{
		public:
			FADEnvelopeGenerator() = default;

			FADEnvelopeGenerator(const FADEnvelopeGenerator& InGenerator)
				: Envelope(InGenerator.Envelope)
				, Value(InGenerator.Value)
				, Params(InGenerator.Params)
			{
			}

			FADEnvelopeGenerator(const FSoundModulationADEnvelopeParams& InParams)
				: Params(InParams)
			{
				Envelope.SetAttackTimeSeconds(InParams.AttackTime);
				Envelope.SetDecayTimeSeconds(InParams.DecayTime);
				Envelope.SetAttackCurveFactor(InParams.AttackCurve);
				Envelope.SetDecayCurveFactor(InParams.DecayCurve);
				Envelope.SetLooping(InParams.bLooping);
				Envelope.Attack();
			}

			virtual ~FADEnvelopeGenerator() = default;

			virtual float GetValue() const override
			{
				return Value;
			}

			virtual bool IsBypassed() const override
			{
				return Params.bBypass;
			}

			virtual void Update(double InElapsed) override
			{
				if (InElapsed > 0.0f)
				{
					const float SampleRate = static_cast<float>(1.0 / InElapsed);
					Envelope.Init(SampleRate);
					Envelope.GetNextEnvelopeOut(Value);
				}
			}

			virtual FGeneratorPtr Clone() const override
			{
				return FGeneratorPtr(new FADEnvelopeGenerator(*this));
			}

			virtual void UpdateGenerator(FGeneratorPtr&& InGenerator) override
			{
				if (!ensure(InGenerator.IsValid()))
				{
					return;
				}

				AudioRenderThreadCommand([this, NewGenerator = MoveTemp(InGenerator)]()
				{
					const FADEnvelopeGenerator* Generator = static_cast<const FADEnvelopeGenerator*>(NewGenerator.Get());
					Params = Generator->Params;

					Envelope.SetAttackTimeSeconds(Params.AttackTime);
					Envelope.SetDecayTimeSeconds(Params.DecayTime);
					Envelope.SetAttackCurveFactor(Params.AttackCurve);
					Envelope.SetDecayCurveFactor(Params.DecayCurve);
					Envelope.SetLooping(Params.bLooping);
				});
			}

#if !UE_BUILD_SHIPPING
			virtual void GetDebugCategories(TArray<FString>& OutDebugCategories) const override
			{
				OutDebugCategories = GeneratorADEnvelopePrivate::DebugCategories;
			}

			virtual const FString& GetDebugName() const override
			{
				return GeneratorADEnvelopePrivate::DebugName;
			}

			virtual void GetDebugValues(TArray<FString>& OutDebugValues) const override
			{
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), GetValue()));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Params.AttackTime));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Params.DecayTime));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Params.AttackCurve));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Params.DecayCurve));
				OutDebugValues.Add(FString::Printf(TEXT("%i"), Envelope.IsLooping()));
			}
#endif // !UE_BUILD_SHIPPING

	protected:
		Audio::FADEnvelope Envelope;
		float Value = 1.0f;
		FSoundModulationADEnvelopeParams Params;
	};
} // namespace AudioModulation

AudioModulation::FGeneratorPtr USoundModulationGeneratorADEnvelope::CreateInstance() const
{
	using namespace AudioModulation;

	return FGeneratorPtr(new FADEnvelopeGenerator(Params));
}

