// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/SoundModulationEnvelopeFollower.h"

#include "Algo/MaxElement.h"
#include "AudioBusSubsystem.h"
#include "AudioDefines.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioModulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationEnvelopeFollower)


namespace AudioModulation
{
	namespace GeneratorEnvelopeFollowerPrivate
	{
		static const TArray<FString> DebugCategories =
		{
			TEXT("Value"),
			TEXT("Gain"),
			TEXT("Attack"),
			TEXT("Release"),
		};

		static const FString DebugName = TEXT("EnvelopeFollower");
	}

	class AUDIOMODULATION_API FEnvelopeFollowerGenerator : public IGenerator
	{
	public:
		FEnvelopeFollowerGenerator(const FEnvelopeFollowerGenerator& InGenerator)
			: AudioBusPatch(InGenerator.AudioBusPatch)
			, TempBuffer(InGenerator.TempBuffer)
			, EnvelopeFollower(InGenerator.EnvelopeFollower)
			, InitParams(InGenerator.InitParams)
			, BusId(InGenerator.BusId)
			, CurrentValue(InGenerator.CurrentValue)
			, Gain(InGenerator.Gain)
			, bBypass(InGenerator.bBypass)
			, bInvert(InGenerator.bInvert)
			, bBusRequiresPatch(InGenerator.bBusRequiresPatch)
		{
			DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &FEnvelopeFollowerGenerator::OnDeviceDestroyed);
		}

		FEnvelopeFollowerGenerator(const FEnvelopeFollowerGeneratorParams& InGeneratorParams)
			: Gain(InGeneratorParams.Gain)
			, bBypass(InGeneratorParams.bBypass ? 1 : 0)
			, bInvert(InGeneratorParams.bInvert ? 1 : 0)
			, bBusRequiresPatch(1)
		{
			DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &FEnvelopeFollowerGenerator::OnDeviceDestroyed);

			if (UAudioBus* AudioBus = InGeneratorParams.AudioBus)
			{
				BusId = AudioBus->GetUniqueID();
				InitParams.NumChannels = AudioBus->GetNumChannels();
			}

			InitParams.AttackTimeMsec = InGeneratorParams.AttackTime * 1000.0f;
			InitParams.ReleaseTimeMsec = InGeneratorParams.ReleaseTime * 1000.0f;
			InitParams.Mode = Audio::EPeakMode::Peak;

			if (bInvert)
			{
				CurrentValue = 1.0f;
			}
		}

		virtual ~FEnvelopeFollowerGenerator()
		{
			FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);

			AudioBusPatch.Reset();
		}

	#if !UE_BUILD_SHIPPING
		virtual void GetDebugCategories(TArray<FString>& OutDebugCategories) const override
		{
			OutDebugCategories = GeneratorEnvelopeFollowerPrivate::DebugCategories;
		}

		virtual const FString& GetDebugName() const override
		{
			return GeneratorEnvelopeFollowerPrivate::DebugName;
		}

		virtual void GetDebugValues(TArray<FString>& OutDebugValues) const override
		{
			const float AttackTime = EnvelopeFollower.GetAttackTimeMsec() / 1000.f;
			const float ReleaseTime = EnvelopeFollower.GetReleaseTimeMsec() / 1000.f;

			OutDebugValues.Add(FString::Printf(TEXT("%.4f"), GetValue()));
			OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Gain));
			OutDebugValues.Add(FString::Printf(TEXT("%.4f"), AttackTime));
			OutDebugValues.Add(FString::Printf(TEXT("%.4f"), ReleaseTime));
		}

	#endif // !UE_BUILD_SHIPPING

		virtual FGeneratorPtr Clone() const override
		{
			return FGeneratorPtr(new FEnvelopeFollowerGenerator(*this));
		}

		virtual void UpdateGenerator(FGeneratorPtr&& InGenerator) override
		{
			if (!ensure(InGenerator.IsValid()))
			{
				return;
			}

			AudioRenderThreadCommand([this, NewGenerator = MoveTemp(InGenerator)]()
			{
				const FEnvelopeFollowerGenerator* Generator = static_cast<const FEnvelopeFollowerGenerator*>(NewGenerator.Get());

				bBypass = Generator->bBypass;
				bInvert = Generator->bInvert;

				EnvelopeFollower.SetAnalog(Generator->InitParams.bIsAnalog);
				EnvelopeFollower.SetAttackTime(Generator->InitParams.AttackTimeMsec);
				EnvelopeFollower.SetMode(Generator->InitParams.Mode);
				EnvelopeFollower.SetNumChannels(Generator->InitParams.NumChannels);
				EnvelopeFollower.SetReleaseTime(Generator->InitParams.ReleaseTimeMsec);

				if (Generator->BusId != BusId || !FMath::IsNearlyEqual(Gain, Generator->Gain))
				{
					BusId = Generator->BusId;
					Gain = Generator->Gain;
					bBusRequiresPatch = true;
				}
			});
		}

		virtual float GetValue() const override
		{
			return CurrentValue;
		}

		virtual void Init(Audio::FDeviceId InDeviceId) override
		{
			if (AudioDeviceId == INDEX_NONE)
			{
				AudioDeviceId = InDeviceId;
				if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
				{
					FAudioDevice* AudioDevice = DeviceManager->GetAudioDeviceRaw(InDeviceId);
					if (Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(AudioDevice))
					{
						InitParams.SampleRate = MixerDevice->SampleRate;
					}
				}

				EnvelopeFollower = Audio::FEnvelopeFollower(InitParams);
			}
		}

		virtual bool IsBypassed() const override
		{
			return bBypass;
		}

		virtual void Update(double InElapsed) override
		{
			if (bBusRequiresPatch)
			{
				CreatePatchForBus();
			}

			if (AudioBusPatch.IsValid())
			{
				const int32 NumSamples = AudioBusPatch->GetNumSamplesAvailable();
				const int32 NumFrames = NumSamples / EnvelopeFollower.GetNumChannels();

				if (NumSamples > 0)
				{
					TempBuffer.Reset();
					TempBuffer.AddZeroed(NumSamples);

					AudioBusPatch->PopAudio(TempBuffer.GetData(), NumSamples, true /* bUseLatestAudio */);

					EnvelopeFollower.ProcessAudio(TempBuffer.GetData(), NumFrames);
				}

				float MaxValue = 0.f;
				if (const float* MaxEnvelopePtr = Algo::MaxElement(EnvelopeFollower.GetEnvelopeValues()))
				{
					MaxValue = FMath::Clamp(*MaxEnvelopePtr, 0.f, 1.f);
				}

				CurrentValue = bInvert ? 1.0f - MaxValue : MaxValue;
			}
			else
			{
				bBusRequiresPatch = true;
				CurrentValue = bInvert ? 1.f : 0.f;
			}

		}

		// Retrieving/mutating the MixerDevice is only safe during Update calls 
		// on the audio render thread if it is not called during FAudioDevice::Teardown, 
		// so it is reset prior to that via the OnDeviceDestroyed callback
		void CreatePatchForBus()
		{
			if (BusId != INDEX_NONE)
			{
				if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
				{
					FAudioDevice* AudioDevice = DeviceManager->GetAudioDeviceRaw(AudioDeviceId);
					if (Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(AudioDevice))
					{
						if (UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>())
						{
							AudioBusPatch = AudioBusSubsystem->AddPatchOutputForAudioBus(Audio::FAudioBusKey(BusId), MixerDevice->GetNumOutputFrames(), EnvelopeFollower.GetNumChannels(), Gain);
							bBusRequiresPatch = false;
						}
					}
				}
			}
		}

		void OnDeviceDestroyed(Audio::FDeviceId InDeviceId)
		{
			if (InDeviceId == AudioDeviceId)
			{
				// Reset bus id on device destruction to avoid reinitializing
				// it during FAudioDevice::Teardown via Update.
				BusId = INDEX_NONE;
				FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
			}
		}

	private:
		Audio::FPatchOutputStrongPtr AudioBusPatch;
		Audio::FAlignedFloatBuffer TempBuffer;
		Audio::FEnvelopeFollower EnvelopeFollower;

		Audio::FEnvelopeFollowerInitParams InitParams;

		uint32 BusId = INDEX_NONE;
		float CurrentValue = 0.0f;
		float Gain = 1.0f;
		uint8 bBypass : 1;
		uint8 bInvert : 1;
		uint8 bBusRequiresPatch : 1;

		FDelegateHandle DeviceDestroyedHandle;
	};
} // namespace AudioModulation

AudioModulation::FGeneratorPtr USoundModulationGeneratorEnvelopeFollower::CreateInstance() const
{
	using namespace AudioModulation;

	return FGeneratorPtr(new FEnvelopeFollowerGenerator(Params));
}

