// Copyright Epic Games, Inc. All Rights Reserved.
#include "Generators/SoundModulationLFO.h"

#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationSystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationLFO)


namespace AudioModulation
{
	namespace GeneratorLFOPrivate
	{
		static const TArray<FString> DebugCategories =
		{
			TEXT("Value"),
			TEXT("Gain"),
			TEXT("Frequency"),
			TEXT("Offset"),
			TEXT("Curve")
		};

		static const FString DebugName = TEXT("LFO");
	}

	class FLFOGenerator : public IGenerator
	{
		public:
			FLFOGenerator() = default;

			FLFOGenerator(const FLFOGenerator& InGenerator)
				: LFO(InGenerator.LFO)
				, Value(InGenerator.Value)
				, Params(InGenerator.Params)
			{
			}

			FLFOGenerator(const FSoundModulationLFOParams& InParams)
				: Params(InParams)
			{
				LFO.SetBipolar(false);
				LFO.SetExponentialFactor(Params.ExponentialFactor);
				LFO.SetFrequency(Params.Frequency);
				LFO.SetGain(Params.Amplitude);
				LFO.SetMode(Params.bLooping ? Audio::ELFOMode::Type::Sync : Audio::ELFOMode::OneShot);
				LFO.SetPulseWidth(Params.Width);
				LFO.SetPhaseOffset(Params.Phase);
				LFO.SetType(static_cast<Audio::ELFO::Type>(Params.Shape));
				LFO.Start();

				static_assert(static_cast<int32>(ESoundModulationLFOShape::COUNT) == static_cast<int32>(Audio::ELFO::Type::NumLFOTypes), "LFOShape/ELFO Type mismatch");
			}

			virtual ~FLFOGenerator() = default;

			virtual float GetValue() const override
			{
				return Value;
			}

			virtual bool IsBypassed() const override
			{
				return Params.bBypass;
			}

			virtual void Update(double InElapsed) override
			{
				if (InElapsed > 0.0f && LFO.GetFrequency() > 0.0f)
				{
					const float SampleRate = static_cast<float>(1.0 / InElapsed);
					LFO.SetSampleRate(SampleRate);
					LFO.Update();
					Value = LFO.Generate() + Params.Offset;
				}
			}

			virtual FGeneratorPtr Clone() const override
			{
				return FGeneratorPtr(new FLFOGenerator(*this));
			}

			virtual void UpdateGenerator(FGeneratorPtr&& InGenerator) override
			{
				if (!ensure(InGenerator.IsValid()))
				{
					return;
				}

				AudioRenderThreadCommand([this, NewGenerator = MoveTemp(InGenerator)]()
				{
					const FLFOGenerator* Generator = static_cast<const FLFOGenerator*>(NewGenerator.Get());
					Params = Generator->Params;

					LFO.SetExponentialFactor(Params.ExponentialFactor);
					LFO.SetFrequency(Params.Frequency);
					LFO.SetGain(Params.Amplitude);
					LFO.SetPhaseOffset(Params.Phase);
					LFO.SetPulseWidth(Params.Width);

					static_assert(static_cast<int32>(ESoundModulationLFOShape::COUNT) == static_cast<int32>(Audio::ELFO::Type::NumLFOTypes), "LFOShape/ELFO Type mismatch");
					LFO.SetType(static_cast<Audio::ELFO::Type>(Params.Shape));

					Audio::ELFOMode::Type NewMode = Params.bLooping ? Audio::ELFOMode::Type::Sync : Audio::ELFOMode::OneShot;
					const bool bModeUpdated = NewMode != LFO.GetMode();
					if (bModeUpdated)
					{
						LFO.SetMode(NewMode);
					}

					if (bModeUpdated)
					{
						LFO.Start();
					}
				});
			}

#if !UE_BUILD_SHIPPING
			virtual void GetDebugCategories(TArray<FString>& OutDebugCategories) const override
			{
				OutDebugCategories = GeneratorLFOPrivate::DebugCategories;
			}

			virtual const FString& GetDebugName() const override
			{
				return GeneratorLFOPrivate::DebugName;
			}

			virtual void GetDebugValues(TArray<FString>& OutDebugValues) const override
			{
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), GetValue()));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), LFO.GetGain()));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), LFO.GetFrequency()));
				OutDebugValues.Add(FString::Printf(TEXT("%.4f"), Params.Offset));

				switch (LFO.GetType())
				{
					case Audio::ELFO::DownSaw:
						OutDebugValues.Add(TEXT("DownSaw"));
						break;

					case Audio::ELFO::Exponential:
						OutDebugValues.Add(TEXT("Exponential"));
						break;

					case Audio::ELFO::RandomSampleHold:
						OutDebugValues.Add(TEXT("Random (Sample & Hold)"));
						break;

					case Audio::ELFO::Sine:
						OutDebugValues.Add(TEXT("Sine"));
						break;

					case Audio::ELFO::Square:
						OutDebugValues.Add(TEXT("Square"));
						break;

					case Audio::ELFO::Triangle:
						OutDebugValues.Add(TEXT("Triangle"));
						break;

					case Audio::ELFO::UpSaw:
						OutDebugValues.Add(TEXT("Up Saw"));
						break;

					default:
						static_assert(static_cast<int32>(Audio::ELFO::NumLFOTypes) == 7, "Missing LFO type case coverage");
						break;
				}
			}
#endif // !UE_BUILD_SHIPPING

	protected:
		Audio::FLFO LFO;
		float Value = 1.0f;
		FSoundModulationLFOParams Params;
	};
} // namespace AudioModulation

AudioModulation::FGeneratorPtr USoundModulationGeneratorLFO::CreateInstance() const
{
	using namespace AudioModulation;

	return FGeneratorPtr(new FLFOGenerator(Params));
}

