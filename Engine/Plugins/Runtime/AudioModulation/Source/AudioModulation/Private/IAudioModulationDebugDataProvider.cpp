// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioModulationDebugDataProvider.h"

#if WITH_AUDIOMODULATION
#if !UE_BUILD_SHIPPING

void AudioModulation::IDebugDataProvider::GetControlBusDebugInfo(TArray<FControlBusDebugInfo>& OutDebugInfo) {}
void AudioModulation::IDebugDataProvider::GetControlBusMixDebugInfo(TArray<FControlBusMixDebugInfo>& OutDebugInfo) {}

#endif // !UE_BUILD_SHIPPING
#endif // WITH_AUDIOMODULATION

