// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
#include "AudioDefines.h"
#include "AudioModulation.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "SoundModulatorAsset.h"

#define LOCTEXT_NAMESPACE "AudioModulationNodes"

namespace AudioModulation
{
	class FGetModulatorValueNodeOperator : public Metasound::TExecutableOperator<FGetModulatorValueNodeOperator>
	{
	public:
		static const Metasound::FVertexInterface& GetDefaultInterface()
		{
			using namespace Metasound;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FSoundModulatorAsset>("Modulator", FDataVertexMetadata{ LOCTEXT("MetasoundValueNode_InputModulatorName", "Modulator") }),
					TInputDataVertex<bool>("Normalized", FDataVertexMetadata{LOCTEXT("MixModulatorsNode_InputNormalizedName", "Normalized")}, true )
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>("Out", FDataVertexMetadata{LOCTEXT("MetasoundValueNode_OutputModulatorValue", "Out")})
				)
			);

			return DefaultInterface;
		}

		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Metadata
				{
					Metasound::FNodeClassName { "Modulation", "GetModulatorValue", "" },
					1, // Major Version
					0, // Minor Version
					LOCTEXT("GetModulatorValueNode_Name", "Get Modulator Value"),
					LOCTEXT("GetModulatorValueNode_Description", "Returns the current value of the given modulator. Converts value to unit space if 'Normalized' is false."),
					AudioModulation::PluginAuthor,
					AudioModulation::PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ AudioModulation::PluginNodeCategory },
					{ },
					{ }
				};

				return Metadata;
			};

			static const Metasound::FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<Metasound::IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			using namespace Metasound;

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			if (InParams.Environment.Contains<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			{
				FSoundModulatorAssetReadRef ModulatorReadRef = InputData.GetOrCreateDefaultDataReadReference<FSoundModulatorAsset>("Modulator", InParams.OperatorSettings);
				FBoolReadRef NormalizedReadRef = InputData.GetOrCreateDefaultDataReadReference<bool>("Normalized", InParams.OperatorSettings);

				return MakeUnique<FGetModulatorValueNodeOperator>(InParams, ModulatorReadRef, NormalizedReadRef);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Cannot create GetModulatorValueNode when no audio device ID supplied. Expected metasound environment variable '%ls'", *Metasound::Frontend::SourceInterface::Environment::DeviceID.ToString());
				return nullptr;
			}
		}

		FGetModulatorValueNodeOperator(const Metasound::FBuildOperatorParams& InParams, const FSoundModulatorAssetReadRef& InModulator, const Metasound::FBoolReadRef& InNormalized)
			: DeviceId(InParams.Environment.GetValue<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			, Modulator(InModulator)
			, Normalized(InNormalized)
			, OutValue(Metasound::TDataWriteReferenceFactory<float>::CreateAny(InParams.OperatorSettings))
		{
			Execute();
		}

		virtual ~FGetModulatorValueNodeOperator() = default;

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			
			InOutVertexData.BindReadVertex("Modulator", Modulator);
			InOutVertexData.BindReadVertex("Normalized", Normalized);
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			
			InOutVertexData.BindReadVertex("Out", OutValue);
		}

		void Execute()
		{
			const FSoundModulatorAsset& ModulatorAsset = *Modulator;

			// Checks if handle needs refresh.  This can happen if the incoming modulator is changed or nulled.
			if (ModulatorAsset.GetModulatorId() != ModHandle.GetModulatorId())
			{
				if (IAudioModulationManager* InModulation = AudioModulation::GetDeviceModulationManager(DeviceId))
				{
					ModHandle = ModulatorAsset->CreateModulatorHandle(*InModulation);
				}
			}

			float Value = 1.0f;
			if (ModHandle.IsValid())
			{
				ModHandle.GetValueThreadSafe(Value);
			}

			if (!*Normalized)
			{
				const Audio::FModulationParameter& Parameter = ModHandle.GetParameter();
				if (Parameter.bRequiresConversion)
				{
					Parameter.UnitFunction(Value);
				}
			}

			*OutValue = Value;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			if (InParams.Environment.Contains<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			{
				Audio::FDeviceId PriorDeviceId = DeviceId;
				DeviceId = InParams.Environment.GetValue<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID);

				if (PriorDeviceId != DeviceId)
				{
					// Reset mod handle if device ID is altered. Modulator handle IDs 
					// are not unique across devices.
					ModHandle = Audio::FModulatorHandle();
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Missing audio device ID environment variable (%ls) required to properly configure node (Modulation.GetModulatorValue)", *Metasound::Frontend::SourceInterface::Environment::DeviceID.ToString());
			}
			Execute();
		}

	private:
		Audio::FDeviceId DeviceId;

		Audio::FModulatorHandle ModHandle;

		FSoundModulatorAssetReadRef Modulator;
		Metasound::FBoolReadRef Normalized;
		Metasound::TDataWriteReference<float> OutValue;
	};

	using FGetModulatorValueNode = Metasound::TNodeFacade<FGetModulatorValueNodeOperator>;

	METASOUND_REGISTER_NODE(FGetModulatorValueNode)
} // namespace AudioModulation

#undef LOCTEXT_NAMESPACE
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT
