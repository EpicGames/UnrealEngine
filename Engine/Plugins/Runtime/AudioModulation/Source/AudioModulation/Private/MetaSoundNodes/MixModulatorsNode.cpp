// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT

#include "AudioModulation.h"
#include "IAudioModulation.h"
#include "Interfaces/MetasoundFrontendSourceInterface.h"
#include "MetasoundDataFactory.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "SoundModulatorAsset.h"

#define LOCTEXT_NAMESPACE "AudioModulationNodes"

namespace AudioModulation
{
	class FMixModulatorsNodeOperator : public Metasound::TExecutableOperator<FMixModulatorsNodeOperator>
	{
	public:
		static const Metasound::FVertexInterface& GetDefaultInterface()
		{
			using namespace Metasound;

			static const FVertexInterface DefaultInterface(
				FInputVertexInterface(
					TInputDataVertex<FSoundModulatorAsset>("In1", FDataVertexMetadata{ LOCTEXT("MixModulatorsNode_InputModulator1Description", "First modulator to mix together") }),
					TInputDataVertex<FSoundModulatorAsset>("In2", FDataVertexMetadata{ LOCTEXT("MixModulatorsNode_InputModulator2Description", "Second modulator to mix together") }),
					TInputDataVertex<FSoundModulationParameterAsset>("MixParameter", FDataVertexMetadata{ LOCTEXT("MixModulatorsNode_InputMixParameterDescription", "Parameter to use to mix parameters together") }),
					TInputDataVertex<bool>("Normalized", FDataVertexMetadata{ LOCTEXT("MixModulatorsNode_InputNormalizedDescription", "Whether the output value should be normalized [0.0, 1.0] or converted to the unit described by the parameter.") }, true)
				),
				FOutputVertexInterface(
					TOutputDataVertex<float>("Out", FDataVertexMetadata{ LOCTEXT("MixModulatorsNode_OutputModulatorValue", "Out") })
				)
			);

			return DefaultInterface;
		}

		static const Metasound::FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> Metasound::FNodeClassMetadata
			{
				Metasound::FNodeClassMetadata Metadata
				{
					Metasound::FNodeClassName { "Modulation", "MixModulators", "" },
					1, // Major Version
					1, // Minor Version
					LOCTEXT("MixModulatorsNode_Name", "Mix Modulators"),
					LOCTEXT("MixModulatorsNode_Description", "Mixes two modulators using the parameterized mix function. Returns the 'Normalized' value (0-1) if true, or the value in unit space (dB, Frequency, etc.) if set to false."),
					AudioModulation::PluginAuthor,
					AudioModulation::PluginNodeMissingPrompt,
					GetDefaultInterface(),
					{ AudioModulation::PluginNodeCategory },
					{ },
					{ }
				};

				return Metadata;
			};

			static const Metasound::FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<Metasound::IOperator> CreateOperator(const Metasound::FBuildOperatorParams& InParams, Metasound::FBuildResults& OutResults)
		{
			using namespace Metasound;
			const FInputVertexInterfaceData& InputData = InParams.InputData;
				
			if (InParams.Environment.Contains<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			{
				FSoundModulatorAssetReadRef Modulator1ReadRef = InputData.GetOrCreateDefaultDataReadReference<FSoundModulatorAsset>("In1", InParams.OperatorSettings);
				FSoundModulatorAssetReadRef Modulator2ReadRef = InputData.GetOrCreateDefaultDataReadReference<FSoundModulatorAsset>("In2", InParams.OperatorSettings);
				FSoundModulationParameterAssetReadRef ParameterReadRef = InputData.GetOrCreateDefaultDataReadReference<FSoundModulationParameterAsset>("MixParameter", InParams.OperatorSettings);
				FBoolReadRef NormalizedReadRef = InputData.GetOrCreateDefaultDataReadReference<bool>("Normalized", InParams.OperatorSettings);

				return MakeUnique<FMixModulatorsNodeOperator>(InParams, Modulator1ReadRef, Modulator2ReadRef, ParameterReadRef, NormalizedReadRef);
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Cannot create mix modulators node when no audio device ID supplied. Expected metasound environment variable '%ls'", *Metasound::Frontend::SourceInterface::Environment::DeviceID.ToString());
				return nullptr;
			}
		}

		FMixModulatorsNodeOperator(
			const Metasound::FBuildOperatorParams& InParams,
			const FSoundModulatorAssetReadRef& InModulator1,
			const FSoundModulatorAssetReadRef& InModulator2,
			const FSoundModulationParameterAssetReadRef& InParameter,
			const Metasound::FBoolReadRef& InNormalized)
			: DeviceId(InParams.Environment.GetValue<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			, Modulator1(InModulator1)
			, Modulator2(InModulator2)
			, Normalized(InNormalized)
			, Parameter(InParameter)
			, OutValue(Metasound::TDataWriteReferenceFactory<float>::CreateAny(InParams.OperatorSettings))
		{
			Execute();
		}

		virtual ~FMixModulatorsNodeOperator() = default;

		virtual void BindInputs(Metasound::FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			
			InOutVertexData.BindReadVertex("In1", Modulator1);
			InOutVertexData.BindReadVertex("In2", Modulator2);
			InOutVertexData.BindReadVertex("MixParameter", Parameter);
			InOutVertexData.BindReadVertex("Normalized", Normalized);
		}

		virtual void BindOutputs(Metasound::FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace Metasound;
			
			InOutVertexData.BindReadVertex("Out", OutValue);
		}

		void Execute()
		{
			auto UpdateValue = [](const FSoundModulatorAssetReadRef& ReadRef, Audio::FDeviceId InDeviceId, Audio::FModulatorHandle& OutHandle)
			{
				float Value = 1.0f;
				const FSoundModulatorAsset& ModulatorAsset = *ReadRef;

				// Checks if handle needs refresh.  This can happen if the incoming modulator is changed or nulled.
				if (ModulatorAsset.GetModulatorId() != OutHandle.GetModulatorId())
				{
					if (IAudioModulationManager* InModulation = AudioModulation::GetDeviceModulationManager(InDeviceId))
					{
						OutHandle = ModulatorAsset->CreateModulatorHandle(*InModulation);
					}
				}

				if (OutHandle.IsValid())
				{
					OutHandle.GetValueThreadSafe(Value);
				}

				return Value;
			};

			float Value1 = UpdateValue(Modulator1, DeviceId, ModHandle1);
			const float Value2 = UpdateValue(Modulator2, DeviceId, ModHandle2);

			const FSoundModulationParameterAsset& ParameterAsset = *Parameter;
			if (ParameterAsset.IsValid())
			{
				const Audio::FModulationParameter& ModParam = ParameterAsset->GetParameter();
				ModParam.MixFunction(Value1, Value2);
				if (!*Normalized)
				{
					if (ModParam.bRequiresConversion)
					{
						ModParam.UnitFunction(Value1);
					}
				}
			}
			else
			{
				Audio::FModulationParameter::GetDefaultMixFunction()(Value1, Value2);
			}

			*OutValue = Value1;
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			if (InParams.Environment.Contains<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID))
			{
				Audio::FDeviceId PriorDeviceId = DeviceId;
				DeviceId = InParams.Environment.GetValue<Audio::FDeviceId>(Metasound::Frontend::SourceInterface::Environment::DeviceID);

				if (PriorDeviceId != DeviceId)
				{
					// Reset modulator handles because modulator ids are not unique across devices.
					ModHandle1 = Audio::FModulatorHandle{};
					ModHandle2 = Audio::FModulatorHandle{};
				}
			}
			else
			{
				UE_LOGF(LogMetaSound, Warning, "Missing audio device ID environment variable (%ls) required to properly configure node (Modulation.MixModulators)", *Metasound::Frontend::SourceInterface::Environment::DeviceID.ToString());
			}
			Execute();
		}

	private:

		Audio::FDeviceId DeviceId;

		Audio::FModulatorHandle ModHandle1;
		Audio::FModulatorHandle ModHandle2;

		FSoundModulatorAssetReadRef Modulator1;
		FSoundModulatorAssetReadRef Modulator2;
		Metasound::FBoolReadRef Normalized;
		FSoundModulationParameterAssetReadRef Parameter;

		Metasound::TDataWriteReference<float> OutValue;
	};

	using FMixModulatorsNode = Metasound::TNodeFacade<FMixModulatorsNodeOperator>;
	METASOUND_REGISTER_NODE(FMixModulatorsNode)
} // namespace AudioModulation

#undef LOCTEXT_NAMESPACE
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT
