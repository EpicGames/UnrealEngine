// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusMixSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieScene.h"
#include "MovieSceneAudioModulationComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusMixSection)

UMovieSceneAudioControlBusMixSection::UMovieSceneAudioControlBusMixSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BusValue.SetDefault(false);
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
}

EMovieSceneChannelProxyType UMovieSceneAudioControlBusMixSection::CacheChannelProxy()
{
	// Set up the channel proxy
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR
	FMovieSceneChannelMetaData Data;
	Data.SetIdentifiers(TEXT("Control Bus Mix Value"), FText::FromName(MixBus == nullptr ? TEXT("Control Bus Mix Value") : MixBus->GetFName()));

	Channels.Add(BusValue, Data, TMovieSceneExternalValue<bool>::Make());
#else
	Channels.Add(BusValue);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneAudioControlBusMixSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;
	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	check(BuiltInComponents);
	check(TrackComponents);

	const FGuid ObjectBindingID = Params.GetObjectBindingID();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
		.Add(TrackComponents->AudioControlBusMix, FMovieSceneAudioControlBusMixComponentData{ this })
		.Add(BuiltInComponents->BoolChannel, &BusValue)
	);
}

void UMovieSceneAudioControlBusMixSection::SetMixBus(USoundControlBusMix* NewMixBus)
{
	check(NewMixBus);
	MixBus = NewMixBus;
}
