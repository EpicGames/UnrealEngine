// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusMixSystem.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Engine/EngineTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "MovieSceneAudioModulationComponentTypes.h"
#include "MovieSceneAudioControlBusMixSection.h"
#include "AudioModulationStatics.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusMixSystem)

namespace UE::MovieScene
{
	enum class EPreAnimatedAudioControlBusMixStateType
	{
		EnableMixBus,
		DisableMixBus
	};

	template<typename BaseTraits>
	struct FPreAnimatedAudioControlBusMixStateTraits : BaseTraits
	{
		using KeyType = FObjectKey;
		using StorageType = EPreAnimatedAudioControlBusMixStateType;

		EPreAnimatedAudioControlBusMixStateType CachePreAnimatedValue(FObjectKey InKey)
		{
			return EPreAnimatedAudioControlBusMixStateType::EnableMixBus;
		}

		void RestorePreAnimatedValue(FObjectKey InKey, EPreAnimatedAudioControlBusMixStateType InStateType, const FRestoreStateParams& Params)
		{
			if (USoundControlBusMix* MixBus = Cast<USoundControlBusMix>(InKey.ResolveObjectPtr()))
			{
				UWorld* World = UAudioModulationStatics::GetAudioWorld(MixBus);

				if (World != nullptr)
				{
					switch (InStateType)
					{
					case UE::MovieScene::EPreAnimatedAudioControlBusMixStateType::EnableMixBus:
						UAudioModulationStatics::ActivateBusMix(World, MixBus);
						break;
					case UE::MovieScene::EPreAnimatedAudioControlBusMixStateType::DisableMixBus:
						UAudioModulationStatics::DeactivateBusMix(World, MixBus);
						break;
					default:
						break;
					}
				}
			}
		}
	};

	using FPreAnimatedBoundObjectAudioControlBusMixStateTraits = FPreAnimatedAudioControlBusMixStateTraits<FBoundObjectPreAnimatedStateTraits>;

	struct FPreAnimatedAudioControlBusMixStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedBoundObjectAudioControlBusMixStateTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedAudioControlBusMixStorage> StorageID;
	};
	TAutoRegisterPreAnimatedStorageID<FPreAnimatedAudioControlBusMixStorage> FPreAnimatedAudioControlBusMixStorage::StorageID;

	struct FEvaluateAudioControlBusMix
	{
		UMovieSceneAudioControlBusMixSystem* AudioSystem;
		const FInstanceRegistry* InstanceRegistry;

		FEvaluateAudioControlBusMix(UMovieSceneAudioControlBusMixSystem* InAudioSystem)
			: AudioSystem(InAudioSystem)
		{
			check(AudioSystem != nullptr);
			InstanceRegistry = AudioSystem->GetLinker()->GetInstanceRegistry();
		}

		void ForEachAllocation(
			const FEntityAllocation* Allocation,
			TRead<FMovieSceneEntityID> EntityIDs,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<FMovieSceneAudioControlBusMixComponentData> AudioDatas,
			TRead<bool> BusValue) const
		{
			check(Allocation != nullptr);
			check(EntityIDs != nullptr);
			check(RootInstanceHandles != nullptr);
			check(InstanceHandles != nullptr);
			check(AudioDatas != nullptr);

			const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			const int32 Num = Allocation->Num();
			const bool bWantsRestoreState = Allocation->HasComponent(BuiltInComponents->Tags.RestoreState);

			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneEntityID& EntityID = EntityIDs[Index];
				const FRootInstanceHandle& RootInstanceHandle = RootInstanceHandles[Index];
				const FInstanceHandle& InstanceHandle = InstanceHandles[Index];

				const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);

				bool BusValueData = BusValue[Index];

				Evaluate(EntityID, AudioDatas[Index], Instance, RootInstanceHandle, BusValueData, bWantsRestoreState);
			}
		}

	private:

		void Evaluate(
			const FMovieSceneEntityID& EntityID,
			const FMovieSceneAudioControlBusMixComponentData& AudioData,
			const FSequenceInstance& Instance,
			const FRootInstanceHandle& RootInstanceHandle,
			bool BusValue,
			bool bWantsRestoreState) const
		{
			const UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();
			check(PlaybackContext);
			UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
			check(World);

			check(AudioData.Section);
			USoundControlBusMix* MixBus = AudioData.Section->MixBus;

			check(MixBus);

			AudioModulation::FAudioModulationManager* Manager = UAudioModulationStatics::GetModulation(World);
			check(Manager);

			UMovieSceneAudioControlBusMixSection* BusMixSection = AudioData.Section;
			if (!ensureMsgf(BusMixSection, TEXT("No valid control bus mix section found in control bus track component data!")))
			{
				return;
			}
			FObjectKey SectionKey(BusMixSection);
			const AudioControlBusMixSectionData* SectionData = AudioSystem->GetSectionData(SectionKey);

			const bool IsBusMixActive = Manager->IsBusMixActive(*AudioData.Section->MixBus);

			if (!SectionData)
			{
				AudioSystem->PreAnimatedStorage->BeginTrackingEntity(EntityID, bWantsRestoreState, RootInstanceHandle, MixBus);

				if (IsBusMixActive)
				{
					AudioSystem->PreAnimatedStorage->CachePreAnimatedValue(
						FCachePreAnimatedValueParams(), MixBus,
						[](FObjectKey InKey) { return EPreAnimatedAudioControlBusMixStateType::EnableMixBus; });
				}
				else
				{
					AudioSystem->PreAnimatedStorage->CachePreAnimatedValue(
						FCachePreAnimatedValueParams(), MixBus,
						[](FObjectKey InKey) { return EPreAnimatedAudioControlBusMixStateType::DisableMixBus; });
				}

				AudioSystem->AddSectionData(SectionKey, MixBus, IsBusMixActive);
			}

			if (BusValue && !IsBusMixActive)
			{
				UAudioModulationStatics::ActivateBusMix(World, MixBus);
			}
			else if (!BusValue && IsBusMixActive)
			{
				UAudioModulationStatics::DeactivateBusMix(World, MixBus);
			}
		}
	};
}

using AudioControlBusMixSectionData = UE::MovieScene::AudioControlBusMixSectionData;

UMovieSceneAudioControlBusMixSystem::UMovieSceneAudioControlBusMixSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneAudioModulationComponentTypes::Get()->AudioControlBusMix;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		check(BuiltInComponents != nullptr);

		DefineComponentConsumer(GetClass(), BuiltInComponents->BoolChannel);
	}
}

void UMovieSceneAudioControlBusMixSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	// Evaluate the control bus mix toggles 
	FTaskID EvaluateAudioTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TrackComponents->AudioControlBusMix)
		.Read(BuiltInComponents->BoolResult) // Bus Mix Toggle
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.Schedule_PerAllocation<FEvaluateAudioControlBusMix>(&Linker->EntityManager, TaskScheduler, this);
}

void UMovieSceneAudioControlBusMixSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	FSystemTaskPrerequisites Prereqs;

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TrackComponents->AudioControlBusMix)
		.Read(BuiltInComponents->BoolResult) // Bus Value
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.template Dispatch_PerAllocation<FEvaluateAudioControlBusMix>(&Linker->EntityManager, Prereqs, &Subsequents, this);
}

void UMovieSceneAudioControlBusMixSystem::OnLink()
{
	using namespace UE::MovieScene;

	PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedAudioControlBusMixStorage>();
}

void UMovieSceneAudioControlBusMixSystem::OnUnlink()
{
	EvaluatedBusMixes.Empty();
}

const AudioControlBusMixSectionData* UMovieSceneAudioControlBusMixSystem::GetSectionData(FObjectKey Key) const
{
	const AudioControlBusMixSectionData* data = EvaluatedBusMixes.Find(Key);

	return data;
}

AudioControlBusMixSectionData* UMovieSceneAudioControlBusMixSystem::AddSectionData(FObjectKey Key, TObjectPtr<USoundControlBusMix> ControlBusMix, bool bIsActive)
{
	using namespace UE::MovieScene;
	check(ControlBusMix != nullptr);

	AudioControlBusMixSectionData NewData = AudioControlBusMixSectionData(ControlBusMix, bIsActive);

	return &EvaluatedBusMixes.Add(Key, NewData);
}
