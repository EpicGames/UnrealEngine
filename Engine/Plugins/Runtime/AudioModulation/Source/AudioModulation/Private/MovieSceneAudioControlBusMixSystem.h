// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/ObjectMacros.h"
#include "IAudioModulation.h"
#include "AudioModulation.h"

#include "MovieSceneAudioControlBusMixSystem.generated.h"

namespace UE::MovieScene
{
	struct FPreAnimatedAudioControlBusMixStorage;

	struct AudioControlBusMixSectionData
	{
		TObjectPtr<USoundControlBusMix> ControlBusMix;
		bool ControlBusMixActive;
	};
}

UCLASS()
class UMovieSceneAudioControlBusMixSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	using FInstanceHandle = UE::MovieScene::FInstanceHandle;
	using AudioControlBusMixSectionData = UE::MovieScene::AudioControlBusMixSectionData;

	UMovieSceneAudioControlBusMixSystem(const FObjectInitializer& ObjInit);

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	virtual void OnLink() override;
	virtual void OnUnlink() override;

	const AudioControlBusMixSectionData* GetSectionData(FObjectKey Key) const;
	AudioControlBusMixSectionData* AddSectionData(FObjectKey Key, TObjectPtr<USoundControlBusMix> ControlBusMix, bool bIsActive);

	/** Pre-animated state */
	TSharedPtr<UE::MovieScene::FPreAnimatedAudioControlBusMixStorage> PreAnimatedStorage;

	TMap<FObjectKey, AudioControlBusMixSectionData>  EvaluatedBusMixes;
};

