// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusMixTrack.h"

#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "MovieSceneAudioControlBusMixSection.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusMixTrack)

UMovieSceneAudioControlBusMixTrack::UMovieSceneAudioControlBusMixTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMovieSceneSection* UMovieSceneAudioControlBusMixTrack::AddNewControlBusMix(USoundControlBusMix* BusMix)
{
	check(BusMix);

	// add the section
	UMovieSceneAudioControlBusMixSection* NewSection = Cast<UMovieSceneAudioControlBusMixSection>(CreateNewSection());

	NewSection->SetMixBus(BusMix);

	NewSection->SetRange(TRange<FFrameNumber>::All());
	MixSections.Add(NewSection);

#if WITH_EDITORONLY_DATA
	SetTrackRowDisplayName(FText::FromName(BusMix->GetFName()), NewSection->GetRowIndex());
#endif

	return NewSection;
}

bool UMovieSceneAudioControlBusMixTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneAudioControlBusMixSection::StaticClass();
}

void UMovieSceneAudioControlBusMixTrack::RemoveAllAnimationData()
{
	MixSections.Empty();
}

bool UMovieSceneAudioControlBusMixTrack::HasSection(const UMovieSceneSection& Section) const
{
	return MixSections.Contains(&Section);
}

void UMovieSceneAudioControlBusMixTrack::AddSection(UMovieSceneSection& Section)
{
	MixSections.Add(&Section);
}

void UMovieSceneAudioControlBusMixTrack::RemoveSection(UMovieSceneSection& Section)
{
	MixSections.Remove(&Section);
}

void UMovieSceneAudioControlBusMixTrack::RemoveSectionAt(int32 SectionIndex)
{
	if (SectionIndex >= 0 && SectionIndex < MixSections.Num())
	{
		MixSections.RemoveAt(SectionIndex);
	}
}

bool UMovieSceneAudioControlBusMixTrack::IsEmpty() const
{
	return MixSections.IsEmpty();
}

const TArray<UMovieSceneSection*>& UMovieSceneAudioControlBusMixTrack::GetAllSections() const
{
	return MixSections;
}

bool UMovieSceneAudioControlBusMixTrack::SupportsMultipleRows() const
{
	return true;
}

UMovieSceneSection* UMovieSceneAudioControlBusMixTrack::CreateNewSection()
{
	return NewObject<UMovieSceneAudioControlBusMixSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneAudioControlBusMixTrack::GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const
{
	OutClasses.Add(UMovieSceneSectionAnchorsDecoration::StaticClass());
}

#if WITH_EDITOR
FText UMovieSceneAudioControlBusMixTrack::GetDefaultDisplayName() const
{
	if (MixSections.Num() > 0)
	{
		return FText::FromString(Cast<UMovieSceneAudioControlBusMixSection>(MixSections[0])->MixBus->GetName());
	}

	return NSLOCTEXT("MovieSceneAudioControlBusMixTrack", "DefaultTrackName", "Control Bus Mix");
}

void UMovieSceneAudioControlBusMixTrack::PostLoad()
{
	if (MixSections.Num() > 0 && GetDisplayName().EqualTo(FText::FromString(Cast<UMovieSceneAudioControlBusMixSection>(MixSections[0])->MixBus->GetName())))
	{
		UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>();
		UE_LOGF(LogMovieScene, Display, "Control Bus Mix Track: Removing display name %ls from %ls because it matches default asset name and will not update automatically with asset changes", *MixSections[0]->GetName(), *GetNameSafe(Sequence));
		SetDisplayName(FText::GetEmpty());
	}
	Super::PostLoad();
}

#endif
