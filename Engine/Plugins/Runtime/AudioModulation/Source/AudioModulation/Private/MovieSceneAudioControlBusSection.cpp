// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieScene.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneAudioControlBusTrack.h"
#include "MovieSceneAudioModulationComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusSection)

UMovieSceneAudioControlBusSection::UMovieSceneAudioControlBusSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BusValue.SetDefault(1.f);
	ControlBus = nullptr;

	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
}

EMovieSceneChannelProxyType UMovieSceneAudioControlBusSection::CacheChannelProxy()
{
	// Set up the channel proxy
	FMovieSceneChannelProxyData Channels;
	check(ControlBus != nullptr);

#if WITH_EDITOR
	FMovieSceneChannelMetaData Data;
	Data.SetIdentifiers(TEXT("Control Value"), FText::FromName(ControlBus == nullptr ? TEXT("Control Bus") : ControlBus->GetFName()));

	float MinValue = 0.0f;
	float MaxValue = 1.0f;

	if (ControlBus->Parameter)
	{
		MinValue = ControlBus->Parameter->GetUnitMin();
		MaxValue = ControlBus->Parameter->GetUnitMax();
	}
	else
	{
		UE_LOG(LogMovieScene, Warning, TEXT("The control bus for section %s does not have a parameter."), *GetFName().ToString());
	}

	Data.PropertyMetaData.Add(TEXT("ClampMin"), FString::SanitizeFloat(MinValue));
	Data.PropertyMetaData.Add(TEXT("ClampMax"), FString::SanitizeFloat(MaxValue));

	Channels.Add(BusValue, Data, TMovieSceneExternalValue<float>::Make());
#else
	Channels.Add(BusValue);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneAudioControlBusSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	check(OutImportedEntity);
	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	check(BuiltInComponents);
	check(TrackComponents);

	const FGuid ObjectBindingID = Params.GetObjectBindingID();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
		.Add(TrackComponents->AudioControlBus, FMovieSceneAudioControlBusComponentData{ this })
		.Add(BuiltInComponents->FloatChannel[0], &BusValue) // Float channels have 9 slots to store and send data, so since we have 1 float channel we send data through the initial float channel
	);
}

void UMovieSceneAudioControlBusSection::SetBus(USoundControlBus* NewBus)
{
	check(NewBus);
	ControlBus = NewBus;

	// Set default to be in the min range

	if (NewBus->Parameter)
	{
		BusValue.SetDefault(NewBus->Parameter->GetUnitMin());
	}
	else
	{
		BusValue.SetDefault(0);
	}
}
