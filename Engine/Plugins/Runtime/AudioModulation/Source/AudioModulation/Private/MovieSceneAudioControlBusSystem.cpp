// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusSystem.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Engine/EngineTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "IAudioModulation.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneAudioModulationComponentTypes.h"
#include "SoundControlBus.h"
#include "AudioModulationStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusSystem)

namespace UE::MovieScene
{
	struct FEvaluateAudioControlBus
	{
		UMovieSceneAudioControlBusSystem* AudioSystem;
		const FInstanceRegistry* InstanceRegistry;

		FEvaluateAudioControlBus(UMovieSceneAudioControlBusSystem* InAudioSystem)
			: AudioSystem(InAudioSystem)
		{
			check(AudioSystem != nullptr);
			InstanceRegistry = AudioSystem->GetLinker()->GetInstanceRegistry();
		}

		void ForEachAllocation(
			const FEntityAllocation* Allocation,
			TRead<FMovieSceneEntityID> EntityIDs,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<FMovieSceneAudioControlBusComponentData> AudioDatas,
			TRead<double> BusValue) const
		{
			check(Allocation != nullptr);
			check(EntityIDs != nullptr);
			check(RootInstanceHandles != nullptr);
			check(InstanceHandles != nullptr);
			check(AudioDatas != nullptr);

			const int32 Num = Allocation->Num();

			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneEntityID& EntityID = EntityIDs[Index];
				const FRootInstanceHandle& RootInstanceHandle = RootInstanceHandles[Index];
				const FInstanceHandle& InstanceHandle = InstanceHandles[Index];

				const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);

				double BusValueData = BusValue[Index];

				Evaluate(EntityID, AudioDatas[Index], Instance, RootInstanceHandle, BusValueData);
			}
		}

	private:

		void Evaluate(
			const FMovieSceneEntityID& EntityID,
			const FMovieSceneAudioControlBusComponentData& AudioData,
			const FSequenceInstance& Instance,
			const FRootInstanceHandle& RootInstanceHandle,
			double BusValue) const
		{
			UMovieSceneAudioControlBusSection* AudioSection = AudioData.Section;
			if (!ensureMsgf(AudioSection, TEXT("No valid control bus section found in control bus track component data!")))
			{
				return;
			}

			FInstanceHandle InstanceHandle(Instance.GetInstanceHandle());
			FObjectKey SectionKey(AudioSection);

			// Grab the cached modulator
			const FAudioControlBusSectionData* EvaluationData = AudioSystem->GetAudioControlBusComponentEvaluationData(AudioSection);

			if (AudioData.Section->ControlBus == nullptr)
			{
				return;
			}

			FString MixBusID = FString::Printf(TEXT("%s_%d"), *AudioData.Section->ControlBus.GetFName().ToString(), Instance.GetSequenceID().GetInternalValue());

			const UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();
			check(PlaybackContext);
			UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
			check(World);

			if (!UAudioModulationStatics::GetAudioWorld(World))
			{
				return; // If we have no audio world then control bus mixes do not get created or updated properly
			}

			USoundControlBus* Bus = CastChecked<USoundControlBus>(AudioData.Section->ControlBus);

			// Control busses now use normalized values so we convert from user facing values
			// If a parameter does not exist, we send the user set values as is without converting
			const double NormalisedValue = Bus->Parameter ? Bus->Parameter->ConvertUnitToNormalized(BusValue) : BusValue;

			// If no cached value exists, we need to initialize
			if (!EvaluationData || !EvaluationData->ControlBusMix->IsValidLowLevel() || EvaluationData->ControlBusMix == nullptr)
			{
				if (EvaluationData) // If we have evaluation data but the control mix asset is now invalid, we need to deactivate the unused mix before making a new one
				{
					UAudioModulationStatics::DeactivateBusMixByID(World, EvaluationData->BusMixID);

					AudioSystem->RemoveAudioControlBusComponentEvaluationData(AudioSection);
				}

				if (USoundControlBusMix* NewMix = UAudioModulationStatics::CreateBusMixFromValue(World, Bus->GetFName(), {Bus}, NormalisedValue))
				{
					EvaluationData = AudioSystem->AddAudioControlBusComponentEvaluationData(InstanceHandle, AudioSection, Bus, NewMix);
				}
			}

			ensure(EvaluationData);

			if (EvaluationData != nullptr)
			{
				if (!UAudioModulationStatics::IsControlBusMixActive(World, EvaluationData->ControlBusMix.Get()))
				{
					UAudioModulationStatics::ActivateBusMix(World, EvaluationData->ControlBusMix.Get());
				}

				UAudioModulationStatics::UpdateMixByFilter(World, EvaluationData->ControlBusMix.Get(), Bus->Address, nullptr, Bus->Parameter, NormalisedValue, -1);
			}

			// Mixes are only cleared on stopped and not paused because the sequence is still considered active and applying mixes in a paused state
			if (Instance.GetContext().GetStatus() == EMovieScenePlayerStatus::Stopped)
			{
				// On stop clear mixes to allow any other playback in engine to function without those mixes impacting
				AudioSystem->ClearBusMixes();
			}
		}
	};
}

UMovieSceneAudioControlBusSystem::UMovieSceneAudioControlBusSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneAudioModulationComponentTypes::Get()->AudioControlBus;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		constexpr int FloatChannelCount = 9;

		for (int32 Index = 0; Index < FloatChannelCount; ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}
	}
}

void UMovieSceneAudioControlBusSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	// Evaluate the control bus during play using the first double result value from the float channel
	FTaskID EvaluateAudioTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TrackComponents->AudioControlBus)
		.Read(BuiltInComponents->DoubleResult[0]) // Bus Value
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.Schedule_PerAllocation<FEvaluateAudioControlBus>(&Linker->EntityManager, TaskScheduler, this);
}

void UMovieSceneAudioControlBusSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneAudioModulationComponentTypes* TrackComponents = FMovieSceneAudioModulationComponentTypes::Get();

	FSystemTaskPrerequisites Prereqs;

	// Evaluate the control bus during play using the first double result value from the float channel
	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TrackComponents->AudioControlBus)
		.Read(BuiltInComponents->DoubleResult[0]) // Bus Value
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.template Dispatch_PerAllocation<FEvaluateAudioControlBus>(&Linker->EntityManager, Prereqs, &Subsequents, this);
}

void UMovieSceneAudioControlBusSystem::OnLink()
{
}

void UMovieSceneAudioControlBusSystem::OnUnlink()
{
	ClearBusMixes();
}

void UMovieSceneAudioControlBusSystem::ClearBusMixes()
{
	if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
	{
		if (FAudioDevice* AudioDevice = DeviceManager->GetAudioDeviceFromWorldContext(GetWorld()))
		{
			for (const TPair<FObjectKey, FAudioControlBusSectionData> Modulator : ModulatorsByObjectKey)
			{
				UAudioModulationStatics::DeactivateBusMix(GetWorld(), Modulator.Value.ControlBusMix.Get());
			}
		}
	}
	
	ModulatorsByObjectKey.Reset();
}

const FAudioControlBusSectionData* UMovieSceneAudioControlBusSystem::GetAudioControlBusComponentEvaluationData(const TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey) const
{
	const FAudioControlBusSectionData* data = ModulatorsByObjectKey.Find(SectionKey);

	return data;
}

FAudioControlBusSectionData* UMovieSceneAudioControlBusSystem::AddAudioControlBusComponentEvaluationData(FInstanceHandle InstanceHandle, TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey, USoundModulatorBase* SectionData, USoundControlBusMix* Mix)
{
	using namespace UE::MovieScene;
	check(SectionData != nullptr);

	FAudioControlBusSectionData NewData = FAudioControlBusSectionData(Mix, Mix->GetUniqueID());

	FAudioControlBusSectionData* Result = &ModulatorsByObjectKey.Add(SectionKey, NewData);

	return Result;
}

void UMovieSceneAudioControlBusSystem::RemoveAudioControlBusComponentEvaluationData(TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey)
{
	ModulatorsByObjectKey.Remove(SectionKey);
}
