// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioModulation.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneAudioControlBusSection.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/ObjectMacros.h"
#include "IAudioModulation.h"
#include "SoundControlBusMix.h"

#include "MovieSceneAudioControlBusSystem.generated.h"

USTRUCT()
struct FAudioControlBusSectionData
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TObjectPtr<USoundControlBusMix> ControlBusMix;
	uint32 BusMixID;
};

UCLASS()
class UMovieSceneAudioControlBusSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	using FInstanceHandle = UE::MovieScene::FInstanceHandle;

	UMovieSceneAudioControlBusSystem(const FObjectInitializer& ObjInit);

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	virtual void OnLink() override;
	virtual void OnUnlink() override;

	void ClearBusMixes();

	const FAudioControlBusSectionData* GetAudioControlBusComponentEvaluationData(const TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey) const;
	FAudioControlBusSectionData* AddAudioControlBusComponentEvaluationData(FInstanceHandle InstanceHandle, TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey, USoundModulatorBase* SectionData, USoundControlBusMix* Mix);
	void RemoveAudioControlBusComponentEvaluationData(TObjectPtr<UMovieSceneAudioControlBusSection> SectionKey);

protected:
	UPROPERTY(Transient)
	TMap<TObjectPtr<UMovieSceneAudioControlBusSection>, FAudioControlBusSectionData> ModulatorsByObjectKey;
};