// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioControlBusTrack.h"

#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "MovieSceneAudioControlBusSection.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioControlBusTrack)

UMovieSceneAudioControlBusTrack::UMovieSceneAudioControlBusTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMovieSceneSection* UMovieSceneAudioControlBusTrack::AddNewControlBus(USoundControlBus* Bus)
{
	check(Bus);

	// add the section
	UMovieSceneAudioControlBusSection* NewSection = Cast<UMovieSceneAudioControlBusSection>(CreateNewSection());

	NewSection->SetBus(Bus);

	NewSection->SetRange(TRange<FFrameNumber>::All());
	AudioSections.Add(NewSection);

#if WITH_EDITORONLY_DATA
	SetTrackRowDisplayName(FText::FromName(Bus->GetFName()), NewSection->GetRowIndex());
#endif

	return NewSection;
}

bool UMovieSceneAudioControlBusTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneAudioControlBusSection::StaticClass();
}

void UMovieSceneAudioControlBusTrack::RemoveAllAnimationData()
{
	AudioSections.Empty();
}

bool UMovieSceneAudioControlBusTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AudioSections.Contains(&Section);
}

void UMovieSceneAudioControlBusTrack::AddSection(UMovieSceneSection& Section)
{
	AudioSections.Add(&Section);
}

void UMovieSceneAudioControlBusTrack::RemoveSection(UMovieSceneSection& Section)
{
	AudioSections.Remove(&Section);
}

void UMovieSceneAudioControlBusTrack::RemoveSectionAt(int32 SectionIndex)
{
	AudioSections.RemoveAt(SectionIndex);
}

bool UMovieSceneAudioControlBusTrack::IsEmpty() const
{
	return AudioSections.IsEmpty();
}

const TArray<UMovieSceneSection*>& UMovieSceneAudioControlBusTrack::GetAllSections() const
{
	return AudioSections;
}

bool UMovieSceneAudioControlBusTrack::SupportsMultipleRows() const
{
	return true;
}

UMovieSceneSection* UMovieSceneAudioControlBusTrack::CreateNewSection()
{
	return NewObject<UMovieSceneAudioControlBusSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneAudioControlBusTrack::GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const
{
	OutClasses.Add(UMovieSceneSectionAnchorsDecoration::StaticClass());
}

#if WITH_EDITOR
FText UMovieSceneAudioControlBusTrack::GetDefaultDisplayName() const
{
	ensure(AudioSections.Num() > 0);

	if (AudioSections.Num() > 0)
	{
		const UMovieSceneAudioControlBusSection* Section = Cast<UMovieSceneAudioControlBusSection>(AudioSections[0]);

		check(Section);

		if (Section->ControlBus)
		{
			return FText::FromString(Section->ControlBus->GetName());
		}
	}

	return NSLOCTEXT("MovieSceneAudioControlBusTrack","DefaultTrackName", "Control Bus");
}

void UMovieSceneAudioControlBusTrack::PostLoad()
{
	ensure(AudioSections.Num() > 0);

	if (AudioSections.Num() > 0)
	{
		const UMovieSceneAudioControlBusSection* Section = Cast<UMovieSceneAudioControlBusSection>(AudioSections[0]);

		check(Section);

		if (Section->ControlBus && GetDisplayName().EqualTo(FText::FromString(Section->ControlBus->GetName())))
		{
			UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>();
			UE_LOGF(LogMovieScene, Display, "Control Bus Track: Removing display name %ls from %ls because it matches default asset name and will not update automatically with asset changes", *AudioSections[0]->GetName(), *GetNameSafe(Sequence));
			SetDisplayName(FText::GetEmpty());
		}
	}
	Super::PostLoad();
}

#endif // WITH_EDITOR
