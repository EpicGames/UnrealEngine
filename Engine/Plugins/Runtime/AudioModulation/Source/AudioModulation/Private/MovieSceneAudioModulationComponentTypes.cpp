// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAudioModulationComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"

static bool GMovieSceneAudioModulationComponentTypesDestroyed = false;
static TUniquePtr<FMovieSceneAudioModulationComponentTypes> GMovieSceneAudioModulationComponentTypes;

FMovieSceneAudioModulationComponentTypes::FMovieSceneAudioModulationComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewComponentType(&AudioControlBus, TEXT("Audio Control Bus"), EComponentTypeFlags::CopyToChildren);
	ComponentRegistry->NewComponentType(&AudioControlBusMix, TEXT("Audio Control Bus Mix"), EComponentTypeFlags::CopyToChildren);
}

FMovieSceneAudioModulationComponentTypes::~FMovieSceneAudioModulationComponentTypes()
{
}

FMovieSceneAudioModulationComponentTypes* FMovieSceneAudioModulationComponentTypes::Get()
{
	if (!GMovieSceneAudioModulationComponentTypes.IsValid())
	{
		check(!GMovieSceneAudioModulationComponentTypesDestroyed);
		GMovieSceneAudioModulationComponentTypes.Reset(new FMovieSceneAudioModulationComponentTypes);
	}
	return GMovieSceneAudioModulationComponentTypes.Get();
}

void FMovieSceneAudioModulationComponentTypes::Destroy()
{
	GMovieSceneAudioModulationComponentTypes.Reset();
	GMovieSceneAudioModulationComponentTypesDestroyed = true;
}
