// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlBus.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationStatics.h"
#include "AudioModulationSystem.h"
#include "DSP/BufferVectorOperations.h"
#include "Engine/World.h"
#include "SoundControlBusMix.h"
#include "SoundControlBusProxy.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundControlBus)


USoundControlBus::USoundControlBus(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bBypass(false)
#if WITH_EDITORONLY_DATA
	, bOverrideAddress(false)
#endif // WITH_EDITORONLY_DATA
	, Parameter(nullptr)
{
}

TUniquePtr<Audio::IModulatorSettings> USoundControlBus::CreateProxySettings() const
{
	return TUniquePtr<Audio::IModulatorSettings>(new AudioModulation::FControlBusSettings(*this));
}

#if WITH_EDITOR
void USoundControlBus::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	if (!bOverrideAddress)
	{
		Address = GetName();
	}

	Super::PostDuplicate(DuplicateMode);
}

void USoundControlBus::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (FProperty* Property = InPropertyChangedEvent.Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(USoundControlBus, bOverrideAddress) && !bOverrideAddress)
		{
			Address = GetName();
		}

		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(USoundControlBus, Parameter))
		{
			for (TObjectIterator<USoundControlBusMix> Iter; Iter; ++Iter)
			{
				if (USoundControlBusMix* Mix = *Iter)
				{
					for (FSoundControlBusMixStage& Stage : Mix->MixStages)
					{
						if (Stage.Bus == this)
						{
							float UnitValue = Stage.Value.TargetValue;
							if (Parameter)
							{
								UnitValue = Parameter->ConvertNormalizedToUnit(Stage.Value.TargetValue);
							}

							if (!FMath::IsNearlyEqual(Stage.Value.TargetUnitValue, UnitValue, KINDA_SMALL_NUMBER))
							{
								Stage.Value.TargetUnitValue = UnitValue;
							}
						}
					}
				}
			}
		}

		AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModSystem)
		{
			OutModSystem.UpdateModulator(*this);
		});
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void USoundControlBus::PostInitProperties()
{
	if (!bOverrideAddress)
	{
		Address = GetName();
	}

	Super::PostInitProperties();
}

void USoundControlBus::PostRename(UObject* OldOuter, const FName OldName)
{
	if (!bOverrideAddress)
	{
		Address = GetName();
	}
}
#endif // WITH_EDITOR

void USoundControlBus::BeginDestroy()
{
	using namespace AudioModulation;

	if (UWorld* World = GetWorld())
	{
		FAudioDeviceHandle AudioDevice = World->GetAudioDevice();
		if (AudioDevice.IsValid())
		{
			check(AudioDevice->IsModulationPluginEnabled());
			if (IAudioModulationManager* ModulationInterface = AudioDevice->ModulationInterface.Get())
			{
				FAudioModulationManager* Modulation = static_cast<FAudioModulationManager*>(ModulationInterface);
				check(Modulation);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Modulation->DeactivateBus(*this);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	// Call parent destroy at end to ensure object is in a valid state for the modulation manager to clean up first
	Super::BeginDestroy();
}

const Audio::FModulationMixFunction USoundControlBus::GetMixFunction() const
{
	if (Parameter)
	{
		return Parameter->GetMixFunction();
	}

	return Audio::FModulationParameter::GetDefaultMixFunction();
}

TSharedPtr<Audio::IProxyData> USoundControlBus::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	using namespace AudioModulation;
	return MakeShared<FSoundModulatorAssetProxy>(*this);
}

const Audio::FModulationParameter& USoundControlBus::GetOutputParameter() const
{
#if !UE_BUILD_SHIPPING
	return AudioModulation::GetOrRegisterParameter(Parameter, GetName(), GetClass()->GetName());
#else
	return AudioModulation::GetOrRegisterParameter(Parameter, FString(), FString());
#endif
}

