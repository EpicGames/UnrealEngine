// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlBusMix.h"

#include "Audio/AudioAddressPattern.h"
#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationProfileSerializer.h"
#include "AudioModulationStatics.h"
#include "Engine/World.h"
#include "SoundControlBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundControlBusMix)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioModulation"


FSoundControlBusMixStage::FSoundControlBusMixStage()
	: Bus(nullptr)
{
}

FSoundControlBusMixStage::FSoundControlBusMixStage(USoundControlBus* InBus, const float TargetValue)
	: Bus(InBus)
{
	Value.TargetValue = TargetValue;
}

USoundControlBusMix::USoundControlBusMix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundControlBusMix::BeginDestroy()
{
	using namespace AudioModulation;

	if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
	{
		DeviceManager->IterateOverAllDevices([this](::Audio::FDeviceId DeviceId, FAudioDevice* AudioDevice)
			{
				if (AudioDevice->IsModulationPluginEnabled())
				{
					if (IAudioModulationManager* ModulationInterface = AudioDevice->ModulationInterface.Get())
					{
						FAudioModulationManager* Modulation = static_cast<FAudioModulationManager*>(ModulationInterface);
						check(Modulation);
						Modulation->DeactivateBusMix(*this);
					}
				}
			});
	}

	// Call parent destroy at end to ensure object is in a valid state for the modulation manager to clean up first
	Super::BeginDestroy();
}

void USoundControlBusMix::ActivateMix()
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.ActivateBusMix(*this);
	});
}

void USoundControlBusMix::DeactivateMix()
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.DeactivateBusMix(*this);
	});
}

void USoundControlBusMix::DeactivateAllMixes()
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.DeactivateAllBusMixes();
	});
}

void USoundControlBusMix::LoadMixFromProfile()
{
	const bool bSucceeded = AudioModulation::FProfileSerializer::Deserialize(ProfileIndex, *this);
#if WITH_EDITOR
	if (bSucceeded)
	{
		FNotificationInfo Info(FText::Format(
			LOCTEXT("SoundControlBusMix_LoadSucceeded", "'Control Bus Mix '{0}' profile {1} loaded successfully."),
			FText::FromName(GetFName()),
			FText::AsNumber(ProfileIndex)
		));
		Info.bFireAndForget = true;
		Info.ExpireDuration = 2.0f;
		Info.bUseThrobber = true;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
#endif // WITH_EDITOR

	ActivateMix();
}

#if WITH_EDITOR
void USoundControlBusMix::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	OnPropertyChanged(InPropertyChangedEvent.Property, InPropertyChangedEvent.ChangeType);
	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void USoundControlBusMix::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	OnPropertyChanged(InPropertyChangedEvent.Property, InPropertyChangedEvent.ChangeType);
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}

void USoundControlBusMix::OnPropertyChanged(FProperty* Property, EPropertyChangeType::Type InChangeType)
{
	if (Property)
	{
		if (InChangeType == EPropertyChangeType::Interactive || InChangeType == EPropertyChangeType::ValueSet)
		{
			if ((Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetValue))
				|| (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundControlBusMixStage, Bus)))
			{
				for (FSoundControlBusMixStage& Stage : MixStages)
				{
					if (Stage.Bus)
					{
						float UnitValue = Stage.Value.TargetValue;
						if (USoundModulationParameter* Parameter = Stage.Bus->Parameter)
						{
							// Clamp the normalized values--either to the normalized [0,1] range or to the parameter-specified "unclamped" range.
							// The normalized value is not clamped in the ConvertNormalizedToUnit function below, so we do it here.
							if (!Parameter->IsModulationParameterClamped())
							{
								const TRange<float> Range = Parameter->GetNormalizedLimits();
								Stage.Value.TargetValue = FMath::Clamp(Stage.Value.TargetValue, Range.GetLowerBoundValue(), Range.GetUpperBoundValue());
							}
							else
							{
								Stage.Value.TargetValue = FMath::Clamp(Stage.Value.TargetValue, 0.0f, 1.0f);
							}
							
							UnitValue = Parameter->ConvertNormalizedToUnit(Stage.Value.TargetValue);
						}

						if (!FMath::IsNearlyEqual(Stage.Value.TargetUnitValue, UnitValue, KINDA_SMALL_NUMBER))
						{
							Stage.Value.TargetUnitValue = UnitValue;
						}
					}
				}
			}
		}
	}

	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.UpdateMix(*this);
	});
}
#endif // WITH_EDITOR

void USoundControlBusMix::SaveMixToProfile()
{
	const bool bSucceeded = AudioModulation::FProfileSerializer::Serialize(*this, ProfileIndex);
#if WITH_EDITOR
	if (bSucceeded)
	{
		FNotificationInfo Info(FText::Format(
			LOCTEXT("SoundControlBusMix_SaveSucceeded", "'Control Bus Mix '{0}' profile {1} saved successfully."),
			FText::FromName(GetFName()),
			FText::AsNumber(ProfileIndex)
		));
		Info.bFireAndForget = true;
		Info.ExpireDuration = 2.0f;
		Info.bUseThrobber = true;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
#endif // WITH_EDITOR
}

void USoundControlBusMix::SoloMix()
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.SoloBusMix(*this);
	});
}

#undef LOCTEXT_NAMESPACE // AudioModulation

