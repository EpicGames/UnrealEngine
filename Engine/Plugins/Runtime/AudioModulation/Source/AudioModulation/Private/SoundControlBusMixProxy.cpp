// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlBusMixProxy.h"

#include "Audio/AudioAddressPattern.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationProfileSerializer.h"
#include "AudioModulationStatics.h"
#include "AudioModulationSystem.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"

#define LOCTEXT_NAMESPACE "AudioModulation"


namespace AudioModulation
{
	const FBusMixId InvalidBusMixId = INDEX_NONE;

	FModulatorBusMixStageSettings::FModulatorBusMixStageSettings(const FSoundControlBusMixStage& InStage)
		: TModulatorBase<FBusId>(InStage.Bus->GetPathName(), InStage.Bus->GetUniqueID())
		, Address(InStage.Bus->Address)
		, ParamClassId(INDEX_NONE)
		, ParamId(INDEX_NONE)
		, Value(InStage.Value)
		, BusSettings(FControlBusSettings(*InStage.Bus))
	{
		if (USoundModulationParameter* Parameter = InStage.Bus->Parameter)
		{
			ParamId = Parameter->GetUniqueID();

			UClass* Class = Parameter->GetClass();
			check(Class);
			ParamClassId = Class->GetUniqueID();

			Value.SetCurrentValue(Parameter->Settings.ValueNormalized);
		}
	}

	FModulatorBusMixStageProxy::FModulatorBusMixStageProxy(FModulatorBusMixStageSettings&& InSettings, FAudioModulationSystem& OutModSystem)
		: TModulatorBase<FBusId>(InSettings.BusSettings.GetPath(), InSettings.BusSettings.GetId())
		, Address(MoveTemp(InSettings.Address))
		, ParamClassId(InSettings.ParamClassId)
		, ParamId(InSettings.ParamId)
		, Value(InSettings.Value)
		, BusHandle(FBusHandle::Create(MoveTemp(InSettings.BusSettings), OutModSystem.RefProxies.Buses, OutModSystem))
	{
	}

	FModulatorBusMixSettings::FModulatorBusMixSettings(const USoundControlBusMix& InBusMix)
		: TModulatorBase<FBusMixId>(InBusMix.GetPathName()
		, InBusMix.GetUniqueID())
		, Duration(InBusMix.Duration)
		, bRetriggerOnActivation(InBusMix.bRetriggerOnActivation)
	{
		for (const FSoundControlBusMixStage& Stage : InBusMix.MixStages)
		{
			if (Stage.Bus)
			{
				Stages.Add(FModulatorBusMixStageSettings(Stage));
			}
			else
			{
				UE_LOGF(LogAudioModulation, VeryVerbose,
					"USoundControlBusMix '%ls' has stage with no bus specified. "
						"Mix instance initialized with stage ignored.",
					*InBusMix.GetFullName());
			}
		}
	}

	FModulatorBusMixSettings::FModulatorBusMixSettings(FModulatorBusMixSettings&& InBusMixSettings)
		: TModulatorBase<FBusMixId>(InBusMixSettings.GetPath(), InBusMixSettings.GetId())
		, Stages(MoveTemp(InBusMixSettings.Stages))
		, Duration(InBusMixSettings.Duration)
		, bRetriggerOnActivation(InBusMixSettings.bRetriggerOnActivation)
	{
	}

	FModulatorBusMixProxy::FModulatorBusMixProxy(FModulatorBusMixSettings&& InSettings, FAudioModulationSystem& OutModSystem)
		: TModulatorProxyRefType(InSettings.GetPath(), InSettings.GetId(), OutModSystem)
		, Duration(InSettings.Duration)
		, TimeRemaining(Duration)
		, bRetriggerOnActivation(InSettings.bRetriggerOnActivation)
	{
		SetMixDataAndEnable(MoveTemp(InSettings));
	}

	FModulatorBusMixProxy& FModulatorBusMixProxy::operator=(FModulatorBusMixSettings&& InSettings)
	{
		SetMixDataAndEnable(MoveTemp(InSettings));

		return *this;
	}

	FModulatorBusMixProxy::EStatus FModulatorBusMixProxy::GetStatus() const
	{
		return Status;
	}

	void FModulatorBusMixProxy::Reset()
	{
		Stages.Reset();
	}

	void FModulatorBusMixProxy::SetMixDataAndEnable(FModulatorBusMixSettings&& InSettings)
	{
		SetMixData(MoveTemp(InSettings));
		SetEnabled();
	}

	void FModulatorBusMixProxy::SetEnabled()
	{
		check(ModSystem);

		Status = EStatus::Enabled;
		
		if (Duration >= 0.0f)
		{
			TimeRemaining = FMath::Max(Duration, 0);
		}
		
		if (bRetriggerOnActivation)
		{
			int StageIndex = 0;
			for (TPair<FBusId, FModulatorBusMixStageProxy>& IdProxyPair : Stages)
			{
				FModulatorBusMixStageProxy& Stage = IdProxyPair.Value;
				const float DefaultValue = Stage.BusHandle.FindProxy().GetDefaultValue();
				Stage.Value.SetCurrentValue(DefaultValue);
				Stage.Value.SetActiveFade(FSoundModulationMixValue::EActiveFade::Attack);
				Stage.Value.TargetValue = StageValues[StageIndex];
				++StageIndex;
			}
		}
	}

	void FModulatorBusMixProxy::SetMixData(FModulatorBusMixSettings&& InSettings)
	{
		check(ModSystem);

		Duration = InSettings.Duration;
		bRetriggerOnActivation = InSettings.bRetriggerOnActivation;
		
		// Cache stages to avoid releasing stage state (and potentially referenced bus state) when re-enabling
		FStageMap CachedStages = Stages;
		Stages.Reset();
		StageValues.Reset();
		for (FModulatorBusMixStageSettings& StageSettings : InSettings.Stages)
		{
			const FBusId BusId = StageSettings.GetId();
			FModulatorBusMixStageProxy StageProxy(MoveTemp(StageSettings), *ModSystem);
			if (const FModulatorBusMixStageProxy* CachedStage = CachedStages.Find(BusId))
			{
				const float StageValue = CachedStage->Value.GetCurrentValue();
				StageProxy.Value.SetCurrentValue(CachedStage->Value.GetCurrentValue());
			}

			StageValues.Add(StageSettings.Value.TargetValue);
			Stages.Add(BusId, MoveTemp(StageProxy));
		}
	}

	void FModulatorBusMixProxy::SetMixData(const TArray<FModulatorBusMixStageSettings>& InStages, float InFadeTime, const FString& BusMixName, double InDuration, bool bInRetriggerOnActivation)
	{
		
		if (InDuration >= 0.0f)
		{
			Duration = InDuration;
			TimeRemaining = FMath::Max(Duration, 0);
		}
		
		bRetriggerOnActivation = bInRetriggerOnActivation;
		
		for (const FModulatorBusMixStageSettings& NewStage : InStages)
		{
			const FBusId BusId = NewStage.GetId();
			if (FModulatorBusMixStageProxy* StageProxy = Stages.Find(BusId))
			{
				StageProxy->Value.TargetValue = NewStage.Value.TargetValue;
				StageProxy->Value.AttackTime = NewStage.Value.AttackTime;
				StageProxy->Value.ReleaseTime = NewStage.Value.ReleaseTime;

				// Setting entire mix wipes pre-existing user fade requests
				StageProxy->Value.SetActiveFade(FSoundModulationMixValue::EActiveFade::Override, InFadeTime);
			}
			else
			{
				UE_LOGF(LogAudioModulation, Warning, "Bus '%ls' Not currently applied to Bus Mix '%ls'. Please ensure that all your Mix Profiles have the same Control Buses.", *NewStage.Address, *BusMixName);
			}
		}
	}

	void FModulatorBusMixProxy::SetMixByFilter(const FString& InAddressFilter, uint32 InParamClassId, uint32 InParamId, float InValue, float InFadeTime)
	{
		for (TPair<FBusId, FModulatorBusMixStageProxy>& IdProxyPair : Stages)
		{
			FModulatorBusMixStageProxy& StageProxy = IdProxyPair.Value;
			if (InParamId != INDEX_NONE && StageProxy.ParamId != InParamId)
			{
				continue;
			}

			if (InParamClassId != INDEX_NONE && StageProxy.ParamClassId != InParamClassId)
			{
				continue;
			}

			if (!FAudioAddressPattern::PartsMatch(InAddressFilter, StageProxy.Address))
			{
				continue;
			}

			StageProxy.Value.TargetValue = InValue;
			StageProxy.Value.SetActiveFade(FSoundModulationMixValue::EActiveFade::Override, InFadeTime);
		}
	}

	void FModulatorBusMixProxy::SetStopping()
	{
		if (Status == EStatus::Enabled)
		{
			Status = EStatus::Stopping;
		}
	}

	void FModulatorBusMixProxy::Update(const double InElapsed, FBusProxyMap& OutProxyMap)
	{
		if (Status == EStatus::Enabled && Duration >= 0)
		{
			TimeRemaining -= InElapsed;
			if (TimeRemaining <= 0.0)
			{
				UE_LOGF(LogAudioModulation, Display, "Automatically deactivating mix after %.3f seconds", FMath::Max(Duration, 0));
				SetStopping();
			}
		}
		
		bool bRequestStop = true;
		for (TPair<FBusId, FModulatorBusMixStageProxy>& Stage : Stages)
		{
			FModulatorBusMixStageProxy& StageProxy = Stage.Value;
			FSoundModulationMixValue& MixStageValue = StageProxy.Value;

			if (FControlBusProxy* BusProxy = OutProxyMap.Find(StageProxy.GetId()))
			{
				MixStageValue.Update(InElapsed);

				const float CurrentValue = MixStageValue.GetCurrentValue();
				if (Status == EStatus::Stopping)
				{
					MixStageValue.TargetValue = BusProxy->GetDefaultValue();
					MixStageValue.SetActiveFade(FSoundModulationMixValue::EActiveFade::Release);
					if (!FMath::IsNearlyEqual(MixStageValue.TargetValue, CurrentValue))
					{
						bRequestStop = false;
					}
				}
				else
				{
					bRequestStop = false;
				}
				BusProxy->MixIn(CurrentValue);
			}
		}

		if (bRequestStop)
		{
			Status = EStatus::Stopped;
		}
	}

	double FModulatorBusMixProxy::GetTimeRemaining() const
	{
		if (Duration >= 0)
		{
			return TimeRemaining;
		}
		return -1.0;
	}
} // namespace AudioModulation

#undef LOCTEXT_NAMESPACE // AudioModulation
