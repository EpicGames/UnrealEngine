// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDeviceManager.h"
#include "IAudioModulation.h"
#include "SoundControlBusProxy.h"
#include "SoundModulationProxy.h"
#include "SoundModulationValue.h"


// Forward Declarations
struct FSoundControlBusMixStage;
class USoundControlBusMix;


namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;

	using FBusMixId = uint32;
	extern const FBusMixId InvalidBusMixId;

	class FModulatorBusMixStageSettings : public TModulatorBase<FBusId>
	{
	public:
		FModulatorBusMixStageSettings(const FSoundControlBusMixStage& InStage);

		FString Address;
		uint32 ParamClassId = INDEX_NONE;
		uint32 ParamId = INDEX_NONE;
		FSoundModulationMixValue Value;
		FControlBusSettings BusSettings;
	};

	class FModulatorBusMixSettings : public TModulatorBase<FBusMixId>
	{
	public:
		FModulatorBusMixSettings(const USoundControlBusMix& InBusMix);
		FModulatorBusMixSettings(FModulatorBusMixSettings&& InSettings);

		TArray<FModulatorBusMixStageSettings> Stages;

		double Duration = -1.0;
		bool bRetriggerOnActivation = false;
	};

	class FModulatorBusMixStageProxy : public TModulatorBase<FBusId>
	{
	public:

		FModulatorBusMixStageProxy(FModulatorBusMixStageSettings&& InSettings, FAudioModulationSystem& OutModSystem);

		FString Address;
		uint32 ParamClassId = INDEX_NONE;
		uint32 ParamId = INDEX_NONE;
		FSoundModulationMixValue Value;
		FBusHandle BusHandle;
	};

	class FModulatorBusMixProxy : public TModulatorProxyRefType<FBusMixId, FModulatorBusMixProxy, FModulatorBusMixSettings>
	{
	public:
		enum class EStatus : uint8
		{
			Enabled,
			Stopping,
			Stopped
		};

		FModulatorBusMixProxy(FModulatorBusMixSettings&& InMix, FAudioModulationSystem& InModSystem);

		FModulatorBusMixProxy& operator=(FModulatorBusMixSettings&& InSettings);

		EStatus GetStatus() const;

		// Resets stage map
		void Reset();

		void SetEnabled();
		void SetStopping();
		// Set stage and Mix Config settings. This does not enable/activate the mix; for that, use SetEnabled
		void SetMixData(const TArray<FModulatorBusMixStageSettings>& InStages, float InFadeTime, const FString& BusMixName, double InDuration, bool bInRetriggerOnActivation);

		void SetMixDataAndEnable(FModulatorBusMixSettings&& InSettings);
		
		// Set the stage values of all buses which satisfy the input filter data. If any of the filters (Address, Parameter type, or specific parameter) are satisfied, that stage's value will be set 
		void SetMixByFilter(const FString& InAddressFilter, uint32 InParamClassId, uint32 InParamId, float InValue, float InFadeTime);

		void Update(const double Elapsed, FBusProxyMap& ProxyMap);

		// If the Control Bus Mix is using a timer, returns the amount of time left before deactivation (seconds). Otherwise returns -1.
		double GetTimeRemaining() const;
		
		using FStageMap = TMap<FBusId, FModulatorBusMixStageProxy>;
		FStageMap Stages;

	private:
		EStatus Status = EStatus::Stopped;
		double Duration = -1.0;
		double TimeRemaining = 0.0;
		bool bRetriggerOnActivation = false;

		TArray<float> StageValues;
		
		void SetMixData(FModulatorBusMixSettings&& InSettings);
	};

	using FBusMixProxyMap = TMap<FBusMixId, FModulatorBusMixProxy>;
	using FBusMixHandle = TProxyHandle<FBusMixId, FModulatorBusMixProxy, FModulatorBusMixSettings>;
} // namespace AudioModulation