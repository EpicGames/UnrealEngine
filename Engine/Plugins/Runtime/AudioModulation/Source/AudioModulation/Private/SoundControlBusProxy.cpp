// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlBusProxy.h"

#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationSubsystem.h"
#include "AudioModulationSystem.h"
#include "Engine/World.h"
#include "SoundModulationGenerator.h"

static int32 ForceClampNormalizedValuesCVar = 0;
FAutoConsoleVariableRef CVarForceClampNormalizedValues(
	TEXT("au.Modulation.ForceClampAllModulationValues"),
	ForceClampNormalizedValuesCVar,
	TEXT("Forces all Modulation values to stay in the normalized [0,1] range.\n")
	TEXT("0: Normalized values can go outside [0,1] if the Modulation Parameter allows it. 1: Normalized values are always in [0,1] range."),
	ECVF_Default);

namespace AudioModulation
{
	const FBusId InvalidBusId = INDEX_NONE;

	Audio::FModulatorTypeId FControlBusSettings::Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InModulation) const
	{
		FAudioModulationSystem& ModSystem = static_cast<FAudioModulationManager&>(InModulation).GetSystem();

		return ModSystem.RegisterModulator(HandleId, *this);
	}

	FControlBusProxy::FControlBusProxy(FControlBusSettings&& InSettings, FAudioModulationSystem& InModSystem)
		: TModulatorProxyRefType(InSettings.GetPath(), InSettings.GetId(), InModSystem)
	{
		Init(MoveTemp(InSettings));
	}

	FControlBusProxy& FControlBusProxy::operator =(FControlBusSettings&& InSettings)
	{
		Init(MoveTemp(InSettings));
		return *this;
	}

	float FControlBusProxy::GetDefaultValue() const
	{
		return DefaultValue;
	}

	const TArray<FGeneratorHandle>& FControlBusProxy::GetGeneratorHandles() const
	{
		return GeneratorHandles;
	}

	float FControlBusProxy::GetGeneratorValue() const
	{
		return GeneratorValue;
	}

	float FControlBusProxy::GetMixValue() const
	{
		return MixValue;
	}

	float FControlBusProxy::GetValue() const
	{
		float DefaultMixed = Mix(DefaultValue) * GeneratorValue;

		if (IsParameterClamped())
		{
			return FMath::Clamp(DefaultMixed, 0.0f, 1.0f);
		}
		
		return DefaultMixed;
	}

	FName FControlBusProxy::GetParameterName() const
	{
#if UE_BUILD_SHIPPING
		static FName ParameterName;
#endif // !UE_BUILD_SHIPPING

		return ParameterName;
	}

	void FControlBusProxy::Init(FControlBusSettings&& InSettings)
	{
		check(ModSystem);

		GeneratorValue = 1.0f;
		MixValue = NAN;
		MixFunction = MoveTemp(InSettings.MixFunction);

#if !UE_BUILD_SHIPPING
		ParameterName = InSettings.OutputParameter.ParameterName;
#endif // !UE_BUILD_SHIPPING 

		DefaultValue = FMath::Clamp(InSettings.DefaultValue, 0.0f, 1.0f);
		bBypass = InSettings.bBypass;

		bClampToNormalizedRange = InSettings.OutputParameter.bClampToNormalizedRange;
		
		TArray<FGeneratorHandle> NewHandles;
		for (FModulationGeneratorSettings& GeneratorSettings : InSettings.GeneratorSettings)
		{
			NewHandles.Add(FGeneratorHandle::Create(MoveTemp(GeneratorSettings), ModSystem->RefProxies.Generators, *ModSystem));
		}

		// Move vs. reset and adding to original array to avoid potentially clearing handles (and thus current Generator state)
		// and destroying generators if function is called while reinitializing/updating the modulator
		GeneratorHandles = MoveTemp(NewHandles);
	}

	bool FControlBusProxy::IsBypassed() const
	{
		return bBypass;
	}

	float FControlBusProxy::Mix(float ValueA) const
	{
		// If mix value is NaN, it is uninitialized (effectively, the parent bus is inactive)
		// and therefore not mixable, so just return the second value.
		if (FMath::IsNaN(MixValue))
		{
			return ValueA;
		}

		float OutValue = MixValue;
		MixFunction(OutValue, ValueA);
		return OutValue;
	}

	bool FControlBusProxy::IsParameterClamped() const
	{
		// This cvar forces all values to be clamped, highest priority
		if (ForceClampNormalizedValuesCVar)
		{
			return true;
		}

		// The value set on the original Parameter UObject
		return bClampToNormalizedRange;		
	}

	void FControlBusProxy::MixIn(const float InValue)
	{
		MixValue = Mix(InValue);
	}

	void FControlBusProxy::MixGenerators()
	{
		for (const FGeneratorHandle& Handle: GeneratorHandles)
		{
			if (Handle.IsValid())
			{
				const FModulatorGeneratorProxy& GeneratorProxy = Handle.FindProxy();
				if (!GeneratorProxy.IsBypassed())
				{
					GeneratorValue *= GeneratorProxy.GetValue();
				}
			}
		}
	}

	void FControlBusProxy::Reset()
	{
		GeneratorValue = 1.0f;
		MixValue = NAN;
	}
} // namespace AudioModulation
