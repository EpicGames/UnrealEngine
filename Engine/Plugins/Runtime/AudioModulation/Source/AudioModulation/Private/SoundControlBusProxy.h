// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDeviceManager.h"
#include "IAudioModulation.h"
#include "SoundControlBus.h"
#include "SoundModulationParameter.h"
#include "SoundModulationProxy.h"
#include "SoundModulationGeneratorProxy.h"


namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;

	using FBusId = uint32;
	extern const FBusId InvalidBusId;

	struct FControlBusSettings : public TModulatorBase<FBusId>, public Audio::IModulatorSettings
	{
		bool bBypass;
		float DefaultValue;

		TArray<FModulationGeneratorSettings> GeneratorSettings;
		Audio::FModulationMixFunction MixFunction;
		Audio::FModulationParameter OutputParameter;

		FControlBusSettings(const USoundControlBus& InBus)
			: TModulatorBase<FBusId>(InBus.GetPathName(), InBus.GetUniqueID())
			, bBypass(InBus.bBypass)
			, DefaultValue(InBus.GetDefaultNormalizedValue())
			, MixFunction(InBus.GetMixFunction())
			, OutputParameter(InBus.GetOutputParameter())
		{
			for (const USoundModulationGenerator* Generator : InBus.Generators)
			{
				if (Generator)
				{
					FModulationGeneratorSettings Settings(*Generator);
					GeneratorSettings.Add(MoveTemp(Settings));
				}
			}
		}

		virtual TUniquePtr<IModulatorSettings> Clone() const override
		{
			return TUniquePtr<IModulatorSettings>(new FControlBusSettings(*this));
		}

		virtual Audio::FModulatorId GetModulatorId() const override
		{
			return static_cast<Audio::FModulatorId>(GetId());
		}

		virtual const Audio::FModulationParameter& GetOutputParameter() const override
		{
			return OutputParameter;
		}

		virtual Audio::FModulatorTypeId Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InModulation) const override;
	};

	class FControlBusProxy : public TModulatorProxyRefType<FBusId, FControlBusProxy, FControlBusSettings>
	{
	public:
		FControlBusProxy();
		FControlBusProxy(FControlBusSettings&& InSettings, FAudioModulationSystem& InModSystem);

		FControlBusProxy& operator =(FControlBusSettings&& InSettings);

		float GetDefaultValue() const;
		const TArray<FGeneratorHandle>& GetGeneratorHandles() const;
		float GetGeneratorValue() const;
		float GetMixValue() const;
		float GetValue() const;
		FName GetParameterName() const;
		bool IsBypassed() const;
		void MixIn(const float InValue);
		void MixGenerators();
		void Reset();

	private:
		void Init(FControlBusSettings&& InSettings);
		float Mix(float ValueA) const;
		bool IsParameterClamped() const;
		
		float DefaultValue = 0.0f;
		
		// Cached values
		float GeneratorValue = 1.0f;
		float MixValue = NAN;

		bool bBypass = false;

		bool bClampToNormalizedRange = true;

		Audio::FModulationMixFunction MixFunction;
		TArray<FGeneratorHandle> GeneratorHandles;

#if !UE_BUILD_SHIPPING
		FName ParameterName;
#endif // !UE_BUILD_SHIPPING 

	};

	using FBusProxyMap = TMap<FBusId, FControlBusProxy>;
	using FBusHandle = TProxyHandle<FBusId, FControlBusProxy, FControlBusSettings>;
} // namespace AudioModulation