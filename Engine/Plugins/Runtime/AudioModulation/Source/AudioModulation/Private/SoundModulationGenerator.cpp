// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationGenerator.h"

#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "AudioModulationSystem.h"
#include "Engine/World.h"
#include "SoundModulationGeneratorProxy.h"
#include "Templates/Function.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationGenerator)


namespace AudioModulation
{
	void IGenerator::AudioRenderThreadCommand(TUniqueFunction<void()>&& InCommand)
	{
		CommandQueue.Enqueue(MoveTemp(InCommand));
	}

	void IGenerator::PumpCommands()
	{
		TUniqueFunction<void()> Cmd;
		while (!CommandQueue.IsEmpty())
		{
			CommandQueue.Dequeue(Cmd);
			Cmd();
		}
	}
} // namespace AudioModulation

#if WITH_EDITOR
void USoundModulationGenerator::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	// Guards against slamming the modulation system with changes when using sliders.
	if (InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
		{
			OutModulation.UpdateModulator(*this);
		});
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}
#endif // WITH_EDITOR

TSharedPtr<Audio::IProxyData> USoundModulationGenerator::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	using namespace AudioModulation;
	return MakeShared<FSoundModulatorAssetProxy>(*this);
}

TUniquePtr<Audio::IModulatorSettings> USoundModulationGenerator::CreateProxySettings() const
{
	using namespace AudioModulation;
	return TUniquePtr<Audio::IModulatorSettings>(new FModulationGeneratorSettings(*this));
}

void USoundModulationGenerator::BeginDestroy()
{
	using namespace AudioModulation;

	if (UWorld* World = GetWorld())
	{
		FAudioDeviceHandle AudioDevice = World->GetAudioDevice();
		if (AudioDevice.IsValid())
		{
			check(AudioDevice->IsModulationPluginEnabled());
			if (IAudioModulationManager* ModulationInterface = AudioDevice->ModulationInterface.Get())
			{
				FAudioModulationManager* Modulation = static_cast<FAudioModulationManager*>(ModulationInterface);
				check(Modulation);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Modulation->DeactivateGenerator(*this);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	Super::BeginDestroy();
}

