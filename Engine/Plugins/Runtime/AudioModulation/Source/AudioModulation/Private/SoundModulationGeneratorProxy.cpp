// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationGeneratorProxy.h"

#include "AudioModulation.h"
#include "AudioModulationSystem.h"


namespace AudioModulation
{
	const FGeneratorId InvalidGeneratorId = INDEX_NONE;

	const Audio::FModulationParameter& FModulationGeneratorSettings::GetOutputParameter() const
	{
		// For now, all generators are normalized values (0, 1).
		return Audio::GetDefaultModulationParameter();
	}

	Audio::FModulatorTypeId FModulationGeneratorSettings::Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InModulation) const
	{
		FAudioModulationSystem& ModSystem = static_cast<FAudioModulationManager&>(InModulation).GetSystem();
		return ModSystem.RegisterModulator(HandleId, *this);
	}

	FModulatorGeneratorProxy::FModulatorGeneratorProxy(FModulationGeneratorSettings&& InSettings, FAudioModulationSystem& InModSystem)
		: TModulatorProxyRefType(InSettings.GetPath(), InSettings.GetId(), InModSystem)
		, Generator(MoveTemp(InSettings.Generator))
	{
		Generator->Init(InModSystem.GetAudioDeviceId());
	}

	FModulatorGeneratorProxy& FModulatorGeneratorProxy::operator =(FModulationGeneratorSettings&& InSettings)
	{
		if (ensure(InSettings.Generator.IsValid()))
		{
			Generator->UpdateGenerator(MoveTemp(InSettings.Generator));
		}
		else
		{
			Generator.Reset();
		}

		return *this;
	}
} // namespace AudioModulation
