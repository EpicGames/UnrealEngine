// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioDeviceManager.h"
#include "IAudioModulation.h"
#include "SoundModulationGenerator.h"
#include "SoundModulationProxy.h"


namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;
	class FModulatorGeneratorProxy;

	struct FModulationGeneratorSettings;

	// Modulator Ids
	using FGeneratorId = uint32;
	extern const FGeneratorId InvalidGeneratorId;

	using FGeneratorProxyMap = TMap<FGeneratorId, FModulatorGeneratorProxy>;
	using FGeneratorHandle = TProxyHandle<FGeneratorId, FModulatorGeneratorProxy, FModulationGeneratorSettings>;

	struct FModulationGeneratorSettings : public TModulatorBase<FGeneratorId>, public Audio::IModulatorSettings
	{
		FGeneratorPtr Generator;

		FModulationGeneratorSettings() = default;

		FModulationGeneratorSettings(const USoundModulationGenerator& InGenerator)
			: TModulatorBase<FGeneratorId>(InGenerator.GetPathName(), InGenerator.GetUniqueID())
			, Generator(InGenerator.CreateInstance())
		{
		}

		FModulationGeneratorSettings(const FModulationGeneratorSettings& InSettings)
			: TModulatorBase<FGeneratorId>(InSettings.GetPath(), InSettings.GetId())
			, Generator(InSettings.Generator.IsValid() ? InSettings.Generator->Clone() : nullptr)
		{
		}

		FModulationGeneratorSettings(FModulationGeneratorSettings&& InSettings)
			: TModulatorBase<FGeneratorId>(InSettings.GetPath(), InSettings.GetId())
			, Generator(MoveTemp(InSettings.Generator))
		{
		}

		virtual TUniquePtr<IModulatorSettings> Clone() const override
		{
			return TUniquePtr<IModulatorSettings>(new FModulationGeneratorSettings(*this));
		}


		virtual Audio::FModulatorId GetModulatorId() const override
		{
			return static_cast<Audio::FModulatorId>(GetId());
		}

		virtual const Audio::FModulationParameter& GetOutputParameter() const override;
		virtual Audio::FModulatorTypeId Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InModulation) const override;
	};

	class FModulatorGeneratorProxy : public TModulatorProxyRefType<FGeneratorId, FModulatorGeneratorProxy, FModulationGeneratorSettings>
	{
		FGeneratorPtr Generator;

	public:
		FModulatorGeneratorProxy() = default;
		FModulatorGeneratorProxy(FModulationGeneratorSettings&& InSettings, FAudioModulationSystem& InModSystem);
		
		FModulatorGeneratorProxy& operator =(FModulationGeneratorSettings&& InSettings);

		float GetValue() const
		{
			return Generator->GetValue();
		}

		void Init(Audio::FDeviceId InDeviceId)
		{
			Generator->Init(InDeviceId);
		}

		bool IsBypassed() const
		{
			return Generator->IsBypassed();
		}

		void Update(double InElapsed)
		{
			Generator->Update(InElapsed);
		}

		void PumpCommands()
		{
			Generator->PumpCommands();
		}

#if !UE_BUILD_SHIPPING
		TArray<FString> GetDebugCategories() const
		{
			TArray<FString> DebugCategories;
			DebugCategories.Add(TEXT("Name"));
			DebugCategories.Add(TEXT("Ref Count"));

			TArray<FString> GeneratorCategories;
			Generator->GetDebugCategories(GeneratorCategories);
			DebugCategories.Append(GeneratorCategories);

			return DebugCategories;
		}

		TArray<FString> GetDebugValues() const
		{
			TArray<FString> DebugValues;
			DebugValues.Add(GetName().ToString());
			DebugValues.Add(FString::FormatAsNumber(GetRefCount()));

			Generator->GetDebugValues(DebugValues);

			return DebugValues;
		}

		const FString& GetDebugName() const
		{
			return Generator->GetDebugName();
		}
#endif // !UE_BUILD_SHIPPING
	};
} // namespace AudioModulation