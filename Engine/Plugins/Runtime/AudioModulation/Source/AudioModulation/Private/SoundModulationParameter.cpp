// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationParameter.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioModulation.h"
#include "AudioModulationLogging.h"
#include "IAudioModulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationParameter)


TSharedPtr<Audio::IProxyData> USoundModulationParameter::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	using namespace AudioModulation;
	return MakeShared<FSoundModulationPluginParameterAssetProxy>(this);
}

#if WITH_EDITOR
void USoundModulationParameter::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		const FName AssetName = GetFName();
		if (Audio::IsModulationParameterRegistered(AssetName))
		{
			Audio::FModulationParameter NewParam = CreateParameter();
			Audio::RegisterModulationParameter(NewParam.ParameterName, MoveTemp(NewParam));
		}
	}
}


void USoundModulationParameter::RefreshNormalizedValue()
{
	float NewNormalizedValue = ConvertUnitToNormalized(Settings.ValueUnit);
	if (IsModulationParameterClamped())
	{
		NewNormalizedValue = FMath::Clamp(NewNormalizedValue, 0.0f, 1.0f);
	}
	if (!FMath::IsNearlyEqual(NewNormalizedValue, Settings.ValueNormalized))
	{
		Settings.ValueNormalized = NewNormalizedValue;
	}
}

void USoundModulationParameter::RefreshUnitValue()
{
	float NewUnitValue = ConvertNormalizedToUnit(Settings.ValueNormalized);
	if (IsModulationParameterClamped())
	{
		NewUnitValue = FMath::Clamp(NewUnitValue, GetUnitMin(), GetUnitMax());
	}
	if (!FMath::IsNearlyEqual(NewUnitValue, Settings.ValueUnit))
	{
		Settings.ValueUnit = NewUnitValue;
	}
}
#endif // WITH_EDITOR

Audio::FModulationParameter USoundModulationParameter::CreateParameter() const
{
	Audio::FModulationParameter Parameter;
	Parameter.ParameterName = GetFName();
	Parameter.bRequiresConversion = RequiresUnitConversion();
	Parameter.MixFunction = GetMixFunction();
	Parameter.UnitFunction = GetUnitConversionFunction();
	Parameter.NormalizedFunction = GetNormalizedConversionFunction();
	Parameter.DefaultValue = GetUnitDefault();
	Parameter.MinValue = GetUnitMin();
	Parameter.MaxValue = GetUnitMax();
	Parameter.bClampToNormalizedRange = Settings.bClampToNormalizedRange;
	Parameter.TypeHash = GetTypeHash(this);

#if WITH_EDITORONLY_DATA
	Parameter.UnitDisplayName = Settings.UnitDisplayName;
	Parameter.ClassName = GetClass()->GetFName();
#endif // WITH_EDITORONLY_DATA

	return Parameter;
}

bool USoundModulationParameter::IsModulationParameterClamped() const
{
	bool bClampToNormalizedRange = Settings.bClampToNormalizedRange;
	Audio::GetModulationParameterClampValue(GetFName(), bClampToNormalizedRange);

	return bClampToNormalizedRange;
}

bool USoundModulationParameterFrequencyBase::RequiresUnitConversion() const
{
	return true;
}

Audio::FModulationUnitConversionFunction USoundModulationParameterFrequencyBase::GetUnitConversionFunction() const
{
	return [InUnitMin = GetUnitMin(), InUnitMax = GetUnitMax()](float& InOutValue)
	{
		static const FVector2D Domain(0.0f, 1.0f);
		const FVector2D Range(InUnitMin, InUnitMax);
		InOutValue = Audio::GetLogFrequencyClamped(InOutValue, Domain, Range);
	};
}

Audio::FModulationNormalizedConversionFunction USoundModulationParameterFrequencyBase::GetNormalizedConversionFunction() const
{
	return [InUnitMin = GetUnitMin(), InUnitMax = GetUnitMax()](float& InOutValue)
	{
		static const FVector2D Domain(0.0f, 1.0f);
		const FVector2D Range(InUnitMin, InUnitMax);
		InOutValue = Audio::GetLinearFrequencyClamped(InOutValue, Domain, Range);
	};
}

Audio::FModulationMixFunction USoundModulationParameterHPFFrequency::GetMixFunction() const
{
	return [](float& InOutValue, float InValue)
	{
		InOutValue = FMath::Max(InOutValue, InValue);
	};
}

Audio::FModulationParameter USoundModulationParameterHPFFrequency::CreateDefaultParameter()
{
	UClass* ThisClass = USoundModulationParameterHPFFrequency::StaticClass();
	check(ThisClass);
	USoundModulationParameterHPFFrequency* ClassDefault = ThisClass->GetDefaultObject<USoundModulationParameterHPFFrequency>();
	check(ClassDefault);
	return ClassDefault->CreateParameter();
}

USoundModulationParameterHPFFrequency::USoundModulationParameterHPFFrequency(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Settings.ValueNormalized = 0.0f;

#if WITH_EDITORONLY_DATA
	Settings.ValueUnit = GetUnitDefault();
#endif // WITH_EDITORONLY_DATA
}

Audio::FModulationMixFunction USoundModulationParameterLPFFrequency::GetMixFunction() const
{
	return [](float& InOutValueA, float InValueB)
	{
		InOutValueA = FMath::Min(InOutValueA, InValueB);
	};
}

Audio::FModulationParameter USoundModulationParameterLPFFrequency::CreateDefaultParameter()
{
	UClass* ThisClass = USoundModulationParameterLPFFrequency::StaticClass();
	check(ThisClass);
	USoundModulationParameterLPFFrequency* ClassDefault = ThisClass->GetDefaultObject<USoundModulationParameterLPFFrequency>();
	check(ClassDefault);
	return ClassDefault->CreateParameter();
}

bool USoundModulationParameterScaled::RequiresUnitConversion() const
{
	return true;
}

Audio::FModulationUnitConversionFunction USoundModulationParameterScaled::GetUnitConversionFunction() const
{
	return [InUnitMin = UnitMin, InUnitMax = UnitMax](float& InOutValue)
	{
		InOutValue = FMath::Lerp(InUnitMin, InUnitMax, InOutValue);
	};
}

Audio::FModulationNormalizedConversionFunction USoundModulationParameterScaled::GetNormalizedConversionFunction() const
{
	return [InUnitMin = UnitMin, InUnitMax = UnitMax](float& InOutValue)
	{
		const float Denom = FMath::Max(SMALL_NUMBER, InUnitMax - InUnitMin);
		InOutValue = (InOutValue - InUnitMin) / Denom;
	};
}

float USoundModulationParameterScaled::GetUnitMin() const
{
	return UnitMin;
}

float USoundModulationParameterScaled::GetUnitMax() const
{
	return UnitMax;
}

USoundModulationParameterBipolar::USoundModulationParameterBipolar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Settings.ValueNormalized = 0.5f;

#if WITH_EDITORONLY_DATA
	Settings.ValueUnit = GetUnitDefault();
#endif // WITH_EDITORONLY_DATA
}

bool USoundModulationParameterBipolar::RequiresUnitConversion() const
{
	return true;
}

Audio::FModulationMixFunction USoundModulationParameterBipolar::GetMixFunction() const
{
	return [](float& InOutValueA, float InValueB)
	{
		InOutValueA += InValueB - 0.5f;
	};
}

Audio::FModulationUnitConversionFunction USoundModulationParameterBipolar::GetUnitConversionFunction() const
{
	return [InUnitRange = UnitRange](float& InOutValue)
	{
		InOutValue = (InUnitRange * InOutValue) - (0.5f * InUnitRange);
	};
}

Audio::FModulationNormalizedConversionFunction USoundModulationParameterBipolar::GetNormalizedConversionFunction() const
{
	return [InUnitRange = UnitRange](float& InOutValue)
	{
		InOutValue = 0.5f + (InOutValue / FMath::Max(InUnitRange, SMALL_NUMBER));
	};
}

float USoundModulationParameterBipolar::GetUnitMax() const
{
	return UnitRange * 0.5f;
}

float USoundModulationParameterBipolar::GetUnitMin() const
{
	return UnitRange * -0.5f;
}

Audio::FModulationParameter USoundModulationParameterBipolar::CreateDefaultParameter(float UnitRange)
{
	UClass* ThisClass = USoundModulationParameterBipolar::StaticClass();
	check(ThisClass);
	USoundModulationParameterBipolar* ClassDefault = ThisClass->GetDefaultObject<USoundModulationParameterBipolar>();
	check(ClassDefault);
	
	// Cache the default value for Bipolar Range because we need to change it
	const float DefaultRange = ClassDefault->UnitRange;
	ClassDefault->UnitRange = UnitRange;
	
	Audio::FModulationParameter DefaultParam = ClassDefault->CreateParameter();
	
	// Set the default back to what it was before
	ClassDefault->UnitRange = DefaultRange;

	return DefaultParam;
}

USoundModulationParameterVolume::USoundModulationParameterVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Settings.bShowClampOption = true;
#endif
}

bool USoundModulationParameterVolume::RequiresUnitConversion() const
{
	return true;
}

Audio::FModulationUnitConversionFunction USoundModulationParameterVolume::GetUnitConversionFunction() const
{
	return [InUnitMin = GetUnitMin(), InUnitMax = VolumeLimit, bClampToNormalized = IsModulationParameterClamped()](float& InOutValue)
	{
		InOutValue = InOutValue > 0.0f
			? Audio::ConvertToDecibels(InOutValue)
			: InUnitMin;

		if (bClampToNormalized)
		{
			InOutValue = FMath::Clamp(InOutValue, InUnitMin, 0.0f);
		}
		else
		{
			InOutValue = FMath::Clamp(InOutValue, InUnitMin, InUnitMax);
		}
	};
}

Audio::FModulationNormalizedConversionFunction USoundModulationParameterVolume::GetNormalizedConversionFunction() const
{
	return [InUnitMin = GetUnitMin(), InUnitMax = VolumeLimit, bClampToNormalized = IsModulationParameterClamped()](float& InOutValue)
	{
		InOutValue = InOutValue < InUnitMin || FMath::IsNearlyEqual(InOutValue, InUnitMin)
			? 0.0f
			: Audio::ConvertToLinear(InOutValue);

		if (bClampToNormalized)
		{
			InOutValue = FMath::Clamp(InOutValue, 0, 1.0f);
		}
		else
		{
			InOutValue = FMath::Clamp(InOutValue, 0, Audio::ConvertToLinear(InUnitMax));
		}
	};
}

float USoundModulationParameterVolume::GetUnitMin() const
{
	return MinVolume;
}

float USoundModulationParameterVolume::GetUnitMax() const
{
	return 0.0f;
}

TRange<float> USoundModulationParameterVolume::GetNormalizedLimits() const
{
	return TRange<float>(0.0f, Audio::ConvertToLinear(VolumeLimit));
}

Audio::FModulationParameter USoundModulationParameterVolume::CreateDefaultParameter(float MinUnitVolume)
{
	UClass* ThisClass = USoundModulationParameterVolume::StaticClass();
	check(ThisClass);
	USoundModulationParameterVolume* ClassDefault = ThisClass->GetDefaultObject<USoundModulationParameterVolume>();
	check(ClassDefault);

	// Cache the default value for Bipolar Range because we need to change it
	const float DefaultMinVolume = ClassDefault->MinVolume;
	ClassDefault->MinVolume = MinUnitVolume;

	Audio::FModulationParameter NewParam = ClassDefault->CreateParameter();

	// Set the default back to what it was before
	ClassDefault->MinVolume = DefaultMinVolume;

	return NewParam;
}

#if WITH_EDITOR
void USoundModulationParameterVolume::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	VolumeLimit = FMath::Max(VolumeLimit, MinVolume);
	
	Super::PostEditChangeProperty(InPropertyChangedEvent);
}
#endif

USoundModulationParameterAdditive::USoundModulationParameterAdditive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Settings.ValueNormalized = 0.0f;
}

bool USoundModulationParameterAdditive::RequiresUnitConversion() const
{
	return true;
}

Audio::FModulationMixFunction USoundModulationParameterAdditive::GetMixFunction() const
{
	return [](float& InOutValueA, float InValueB)
	{
		InOutValueA += InValueB;
	};
}

Audio::FModulationUnitConversionFunction USoundModulationParameterAdditive::GetUnitConversionFunction() const
{
	return [InUnitMin = UnitMin, InUnitMax = UnitMax](float& InOutValue)
	{
		InOutValue = FMath::Lerp(InUnitMin, InUnitMax, InOutValue);
	};
}

Audio::FModulationNormalizedConversionFunction USoundModulationParameterAdditive::GetNormalizedConversionFunction() const
{
	return [InUnitMin = UnitMin, InUnitMax = UnitMax](float& InOutValue)
	{
		const float Denom = FMath::Max(SMALL_NUMBER, InUnitMax - InUnitMin);
		InOutValue = (InOutValue - InUnitMin) / Denom;
	};
}

float USoundModulationParameterAdditive::GetUnitMax() const
{
	return UnitMax;
}

float USoundModulationParameterAdditive::GetUnitMin() const
{
	return UnitMin;
}

namespace AudioModulation
{
	const Audio::FModulationParameter& GetOrRegisterParameter(const USoundModulationParameter* InParameter, const FString& InName, const FString& InClassName)
	{
		FName ParamName;
		if (InParameter)
		{
			ParamName = InParameter->GetFName();
			if (!Audio::IsModulationParameterRegistered(ParamName))
			{
				TStringBuilder<128> Breadcrumb;
				if (InClassName.IsEmpty())
				{
					Breadcrumb.Append(*InName);
				}
				else
				{
					Breadcrumb.Append(*InClassName).Append(" '").Append(*InName).Append("'");
				}

				UE_LOGF(LogAudioModulation, Display,
					"Parameter '%ls' not registered.  Registration forced via '%ls'.",
					*ParamName.ToString(),
					*Breadcrumb);

				Audio::RegisterModulationParameter(ParamName, InParameter->CreateParameter());
			}
		}

		// Returns default modulation parameter if no parameter provided.
		if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(ParamName))
		{
			return *Parameter;
		}
		return Audio::GetDefaultModulationParameter();
	}

	FSoundModulationPluginParameterAssetProxy::FSoundModulationPluginParameterAssetProxy(USoundModulationParameter* InParameter)
	{
		Parameter = GetOrRegisterParameter(InParameter, TEXT("FSoundModulationPluginParameterAssetProxy construction"), FString());
	}
}
