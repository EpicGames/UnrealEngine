// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationPatch.h"

#include "AudioDefines.h"
#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationStatics.h"
#include "AudioModulationSystem.h"
#include "SoundControlBus.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatchProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationPatch)


#define LOCTEXT_NAMESPACE "SoundModulationPatch"


USoundModulationPatch::USoundModulationPatch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<Audio::IProxyData> USoundModulationPatch::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	using namespace AudioModulation;
	return MakeShared<FSoundModulatorAssetProxy>(*this);
}

TUniquePtr<Audio::IModulatorSettings> USoundModulationPatch::CreateProxySettings() const
{
	using namespace AudioModulation;
	return TUniquePtr<Audio::IModulatorSettings>(new FModulationPatchSettings(*this));
}

const Audio::FModulationParameter& USoundModulationPatch::GetOutputParameter() const
{
	return AudioModulation::GetOrRegisterParameter(PatchSettings.OutputParameter, GetName(), GetClass()->GetName());
}

#if WITH_EDITOR
void USoundModulationPatch::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.UpdateModulator(*this);
	});

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void USoundModulationPatch::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	AudioModulation::IterateModulationManagers([this](AudioModulation::FAudioModulationManager& OutModulation)
	{
		OutModulation.UpdateModulator(*this);
	});

	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}

#if WITH_EDITORONLY_DATA
void USoundModulationPatch::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		for (FSoundControlModulationInput& Input : PatchSettings.Inputs)
		{
			Input.Transform.VersionTableData();
		}
	}

	Super::Serialize(Ar);
}
#endif // WITH_EDITORONLY_DATA

void USoundModulationPatch::PreSave(FObjectPreSaveContext InSaveContext)
{
	Super::PreSave(InSaveContext);
}
#endif // WITH_EDITOR

FSoundControlModulationInput::FSoundControlModulationInput()
	: bSampleAndHold(0)
{
}

const USoundControlBus* FSoundControlModulationInput::GetBus() const
{
	return Bus;
}

const USoundControlBus& FSoundControlModulationInput::GetBusChecked() const
{
	check(Bus);
	return *Bus;
}
#undef LOCTEXT_NAMESPACE // SoundModulationPatch

