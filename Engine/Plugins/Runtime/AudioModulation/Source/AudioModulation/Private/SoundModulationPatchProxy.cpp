// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationPatchProxy.h"

#include "AudioDefines.h"
#include "AudioDevice.h"
#include "AudioModulation.h"
#include "AudioModulationStatics.h"
#include "AudioModulationSystem.h"
#include "IAudioModulation.h"
#include "SoundControlBusProxy.h"
#include "SoundModulationParameter.h"
#include "SoundModulationProxy.h"


namespace AudioModulation
{
	const FPatchId InvalidPatchId = INDEX_NONE;

	Audio::FModulatorTypeId FModulationPatchSettings::Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InModulation) const
	{
		FAudioModulationSystem& ModSystem = static_cast<FAudioModulationManager&>(InModulation).GetSystem();
		return ModSystem.RegisterModulator(HandleId, *this);
	}

	FModulationInputProxy::FModulationInputProxy(FModulationInputSettings&& InSettings, FAudioModulationSystem& OutModSystem)
		: BusHandle(FBusHandle::Create(MoveTemp(InSettings.BusSettings), OutModSystem.RefProxies.Buses, OutModSystem))
		, Transform(MoveTemp(InSettings.Transform))
		, bSampleAndHold(InSettings.bSampleAndHold)
	{
		if (BusHandle.IsValid())
		{
			const FControlBusProxy& BusProxy = BusHandle.FindProxy();
			ModStageValue = BusProxy.GetValue();
		}
	}

	FModulationOutputProxy::FModulationOutputProxy(float InDefaultValue, const Audio::FModulationMixFunction& InMixFunction)
		: MixFunction(InMixFunction)
		, DefaultValue(InDefaultValue)
	{
	}

	FModulationPatchProxy::FModulationPatchProxy(FModulationPatchSettings&& InSettings, FAudioModulationSystem& OutModSystem)
	{
		Init(MoveTemp(InSettings), OutModSystem);
	}

	void FModulationPatchProxy::Init(FModulationPatchSettings&& InSettings, FAudioModulationSystem& OutModSystem)
	{
		bBypass = InSettings.bBypass;
		DefaultValue = InSettings.OutputParameter.DefaultValue;
		if (InSettings.OutputParameter.bRequiresConversion)
		{
			InSettings.OutputParameter.NormalizedFunction(DefaultValue);
		}

		bClampOutputToNormalizedRange = InSettings.OutputParameter.bClampToNormalizedRange;
		
		// Cache existing proxies to avoid releasing bus state (and potentially referenced bus state) when reinitializing
		const TArray<FModulationInputProxy> CachedProxies = InputProxies;

		InputProxies.Reset();
		for (FModulationInputSettings& Input : InSettings.InputSettings)
		{
			InputProxies.Emplace(MoveTemp(Input), OutModSystem);
		}

		OutputProxy = FModulationOutputProxy(DefaultValue, MoveTemp(InSettings.OutputParameter.MixFunction));
	}

	bool FModulationPatchProxy::IsBypassed() const
	{
		return bBypass;
	}

	float FModulationPatchProxy::GetValue() const
	{
		if (bBypass)
		{
			return OutputProxy.DefaultValue;
		}

		return Value;
	}

	void FModulationPatchProxy::Update()
	{
		Value = DefaultValue;

		float& OutSampleHold = OutputProxy.SampleAndHoldValue;
		if (!OutputProxy.bInitialized)
		{
			OutSampleHold = DefaultValue;
			OutputProxy.bInitialized = true;
		}

		for (FModulationInputProxy& Input : InputProxies)
		{
			if (Input.bSampleAndHold)
			{
				if (!OutputProxy.bInitialized && Input.BusHandle.IsValid())
				{
					const FControlBusProxy& BusProxy = Input.BusHandle.FindProxy();
					if (!BusProxy.IsBypassed())
					{
						Input.ModStageValue = BusProxy.GetValue();
						Input.Transform.Apply(Input.ModStageValue);
						OutputProxy.MixFunction(OutSampleHold, Input.ModStageValue);
					}
				}
			}
			else
			{
				if (Input.BusHandle.IsValid())
				{
					const FControlBusProxy& BusProxy = Input.BusHandle.FindProxy();
					if (!BusProxy.IsBypassed())
					{
						Input.ModStageValue = BusProxy.GetValue();
						Input.Transform.Apply(Input.ModStageValue);
						OutputProxy.MixFunction(Value, Input.ModStageValue);
					}
				}
			}
		}

		OutputProxy.MixFunction(Value, OutSampleHold);
		if (bClampOutputToNormalizedRange)
		{
			Value = FMath::Clamp(Value, 0.0f, 1.0f);
		}
	}

	FModulationPatchRefProxy::FModulationPatchRefProxy()
		: TModulatorProxyRefType()
		, FModulationPatchProxy()
	{
	}

	FModulationPatchRefProxy::FModulationPatchRefProxy(FModulationPatchSettings&& InSettings, FAudioModulationSystem& OutModSystem)
		: TModulatorProxyRefType(InSettings.GetPath(), InSettings.GetId(), OutModSystem)
		, FModulationPatchProxy(MoveTemp(InSettings), OutModSystem)
	{
	}

	FModulationPatchRefProxy& FModulationPatchRefProxy::operator=(FModulationPatchSettings&& InSettings)
	{
		check(ModSystem);
		Init(MoveTemp(InSettings), *ModSystem);
		return *this;
	}
} // namespace AudioModulation