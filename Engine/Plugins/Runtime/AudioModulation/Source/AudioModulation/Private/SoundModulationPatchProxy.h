// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "AudioDeviceManager.h"
#include "IAudioModulation.h"
#include "SoundControlBusProxy.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatch.h"
#include "SoundModulationProxy.h"
#include "Templates/Function.h"


namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;

	using FPatchId = uint32;
	extern const FPatchId InvalidPatchId;

	struct FModulationInputSettings
	{
		FControlBusSettings BusSettings;
		FSoundModulationTransform Transform;
		uint8 bSampleAndHold : 1;

		FModulationInputSettings(const FSoundControlModulationInput& InInput)
			: BusSettings(InInput.GetBusChecked())
			, Transform(InInput.Transform)
			, bSampleAndHold(InInput.bSampleAndHold)
		{
		}

		FModulationInputSettings(FSoundControlModulationInput&& InInput)
			: BusSettings(InInput.GetBusChecked())
			, Transform(MoveTemp(InInput.Transform))
			, bSampleAndHold(InInput.bSampleAndHold)
		{
		}
	};

	/** Modulation input instance */
	class FModulationInputProxy
	{
	public:
		FModulationInputProxy() = default;
		FModulationInputProxy(FModulationInputSettings&& InSettings, FAudioModulationSystem& OutModSystem);

		FBusHandle BusHandle;

		FSoundModulationTransform Transform;

		float ModStageValue = 1.0f;

		bool bSampleAndHold = false;
	};

	/** Patch applied as the final stage of a modulation chain prior to output on the sound level (Always active, never removed) */
	struct FModulationOutputProxy
	{
		FModulationOutputProxy() = default;
		FModulationOutputProxy(float InDefaultValue, const Audio::FModulationMixFunction& InMixFunction);

		/** Whether patch has been initialized or not */
		bool bInitialized = false;

		/** Cached value of sample-and-hold input values */
		float SampleAndHoldValue = 1.0f;

		/** Function used to mix values together */
		Audio::FModulationMixFunction MixFunction;

		/** Default value if no inputs are provided */
		float DefaultValue = 1.0f;
	};

	struct FModulationPatchSettings : public TModulatorBase<FPatchId>, public Audio::IModulatorSettings
	{
		TArray<FModulationInputSettings> InputSettings;
		bool bBypass = true;

		Audio::FModulationParameter OutputParameter;

		FModulationPatchSettings() = default;
		FModulationPatchSettings(const FModulationPatchSettings& InPatchSettings) = default;

		FModulationPatchSettings(const USoundModulationPatch& InPatch)
			: TModulatorBase<FPatchId>(InPatch.GetPathName(), InPatch.GetUniqueID())
			, bBypass(InPatch.PatchSettings.bBypass)
			, OutputParameter(InPatch.GetOutputParameter())
		{
			for (const FSoundControlModulationInput& Input : InPatch.PatchSettings.Inputs)
			{
				if (Input.Bus)
				{
					FModulationInputSettings NewInputSettings(Input);

					// Required to avoid referencing external UObject from settings on non-audio/game threads
					NewInputSettings.Transform.CacheCurve();

					InputSettings.Add(MoveTemp(NewInputSettings));
				}
			}
		}

		virtual TUniquePtr<IModulatorSettings> Clone() const override
		{
			return TUniquePtr<IModulatorSettings>(new FModulationPatchSettings(*this));
		}

		virtual Audio::FModulatorId GetModulatorId() const override
		{
			return static_cast<Audio::FModulatorId>(GetId());
		}

		virtual const Audio::FModulationParameter& GetOutputParameter() const override
		{
			return OutputParameter;
		}

		virtual Audio::FModulatorTypeId Register(Audio::FModulatorHandleId HandleId, IAudioModulationManager& InAudioModulation) const override;
	};

	class FModulationPatchProxy
	{
	public:
		FModulationPatchProxy() = default;
		FModulationPatchProxy(FModulationPatchSettings&& InSettings, FAudioModulationSystem& InModSystem);

		/** Whether or not the patch is bypassed (effectively just returning the default value) */
		bool IsBypassed() const;

		/** Returns the value of the patch */
		float GetValue() const;

		/** Updates the patch value */
		void Update();

	protected:
		void Init(FModulationPatchSettings&& InSettings, FAudioModulationSystem& InModSystem);

	private:
		/** Default value of patch */
		float DefaultValue = 1.0f;

		/** Current value of the patch */
		float Value = 1.0f;

		/** Optional modulation inputs */
		TArray<FModulationInputProxy> InputProxies;

		/** Final output modulation post input combination */
		FModulationOutputProxy OutputProxy;

		/** Bypasses the patch and doesn't update modulation value */
		bool bBypass = true;

		/** Whether or not the final output should be clamped to [0,1], based on the Parameter passed in from FModulationPatchSettings */
		bool bClampOutputToNormalizedRange = true;

		friend class FAudioModulationSystem;
	};

	class FModulationPatchRefProxy : public TModulatorProxyRefType<FPatchId, FModulationPatchRefProxy, FModulationPatchSettings>, public FModulationPatchProxy
	{
	public:
		FModulationPatchRefProxy();
		FModulationPatchRefProxy(FModulationPatchSettings&& InSettings, FAudioModulationSystem& OutModSystem);

		FModulationPatchRefProxy& operator=(FModulationPatchSettings&& InSettings);
	};

	using FPatchProxyMap = TMap<FPatchId, FModulationPatchRefProxy>;
	using FPatchHandle = TProxyHandle<FPatchId, FModulationPatchRefProxy, FModulationPatchSettings>;
} // namespace AudioModulation