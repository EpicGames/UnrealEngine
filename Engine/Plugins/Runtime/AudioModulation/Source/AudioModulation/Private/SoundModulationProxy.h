// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioModulationLogging.h"
#include "CoreMinimal.h"
#include "SoundModulationPatch.h"


namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;

	/*
	 * Handle for all ref-counted proxy types, to be used only on the modulator processing thread (not thread safe).
	 */
	template<typename IdType, typename ProxyType, typename ProxySettings>
	class TProxyHandle
	{
		IdType Id;
		TMap<IdType, ProxyType>* ProxyMap;

	public:
		FORCEINLINE bool IsValid() const
		{
			return static_cast<uint32>(Id) != INDEX_NONE;
		}

		FORCEINLINE IdType GetId() const
		{
			return Id;
		}

		FORCEINLINE ProxyType& FindProxy()
		{
			check(IsValid());
			check(ProxyMap);

			return ProxyMap->FindChecked(Id);
		}

		FORCEINLINE const ProxyType& FindProxy() const
		{
			check(IsValid());
			check(ProxyMap);

			return ProxyMap->FindChecked(Id);
		}

		friend FORCEINLINE uint32 GetTypeHash(const TProxyHandle<IdType, ProxyType, ProxySettings>& InHandle)
		{
			return static_cast<uint32>(InHandle.Id);
		}

		TProxyHandle()
			: Id(static_cast<IdType>(INDEX_NONE))
			, ProxyMap(nullptr)
		{
		}

		TProxyHandle(IdType InId, TMap<IdType, ProxyType>& InProxyMap)
			: Id(InId)
			, ProxyMap(&InProxyMap)
		{
			if (IsValid())
			{
				ProxyType& Proxy = ProxyMap->FindChecked(Id);
				Proxy.IncRef();
			}
		}

		TProxyHandle(const TProxyHandle& InHandle)
			: Id(InHandle.Id)
			, ProxyMap(InHandle.ProxyMap)
		{
			if (IsValid())
			{
				ProxyType& Proxy = ProxyMap->FindChecked(Id);
				Proxy.IncRef();
			}
		}

		TProxyHandle(TProxyHandle&& InHandle)
			: Id(InHandle.Id)
			, ProxyMap(InHandle.ProxyMap)
		{
			// No Inc/DecRef required due to transfer of ownership
			InHandle.Id = static_cast<IdType>(INDEX_NONE);
			InHandle.ProxyMap = nullptr;
		}

		~TProxyHandle()
		{
			if (!ProxyMap || !IsValid())
			{
				return;
			}

			ProxyType& Proxy = ProxyMap->FindChecked(Id);
			const uint32 RefCount = Proxy.DecRef();
			if (RefCount == 0)
			{
#if UE_BUILD_SHIPPING
				UE_LOGF(LogAudioModulation, Verbose, "Proxy released: Id '%u'.", Id);
#else // UE_BUILD_SHIPPING
				UE_LOGF(LogAudioModulation, Verbose, "Proxy '%ls' released: Id '%u'.", *Proxy.GetName().ToString(), Id);
#endif // !UE_BUILD_SHIPPING
				ProxyMap->Remove(Id);
			}
		}

		/*
		 * Creates a handle to a proxy modulation object tracked in the provided InProxyMap if it exists, otherwise returns invalid handle.
		 */
		static TProxyHandle<IdType, ProxyType, ProxySettings> Get(const IdType ObjectId, TMap<IdType, ProxyType>& OutProxyMap)
		{
			if (ProxyType* Proxy = OutProxyMap.Find(ObjectId))
			{
				check(Proxy->ModSystem);
				return TProxyHandle<IdType, ProxyType, ProxySettings>(ObjectId, OutProxyMap);
			} 

			return TProxyHandle<IdType, ProxyType, ProxySettings>();
		}

		/*
		 * Creates a handle to a proxy modulation object tracked in the provided InProxyMap.  Creates new proxy if it doesn't exist.
		 */
		static TProxyHandle<IdType, ProxyType, ProxySettings> Create(ProxySettings&& InSettings, TMap<IdType, ProxyType>& OutProxyMap, FAudioModulationSystem& OutModSystem)
		{
			const IdType ObjectId = static_cast<IdType>(InSettings.GetId());

			TProxyHandle<IdType, ProxyType, ProxySettings> NewHandle = Get(InSettings.GetId(), OutProxyMap);
			if (!NewHandle.IsValid())
			{
				UE_LOGF(LogAudioModulation, Verbose, "Proxy created: Id '%u' for object '%ls'.", ObjectId, *InSettings.GetName().ToString());
				OutProxyMap.Add(ObjectId, ProxyType(MoveTemp(InSettings), OutModSystem));
				NewHandle = TProxyHandle<IdType, ProxyType, ProxySettings>(ObjectId, OutProxyMap);
			}

			return NewHandle;
		}

		static TProxyHandle<IdType, ProxyType, ProxySettings> Create(ProxySettings&& InSettings, TMap<IdType, ProxyType>& OutProxyMap, FAudioModulationSystem& OutModSystem, TFunction<void(ProxyType&)> OnCreateProxy)
		{
			const IdType ObjectId = static_cast<IdType>(InSettings.GetId());
			TProxyHandle<IdType, ProxyType, ProxySettings> NewHandle = Get(InSettings.GetId(), OutProxyMap);
			if (!NewHandle.IsValid())
			{
				UE_LOGF(LogAudioModulation, Verbose, "Proxy created: Id '%u' for object '%ls'.", NewHandle.Id, *InSettings.GetName());
				OutProxyMap.Add(ObjectId, ProxyType(MoveTemp(InSettings), OutModSystem));
				NewHandle = TProxyHandle<IdType, ProxyType, ProxySettings>(ObjectId, OutProxyMap);
				OnCreateProxy(NewHandle.FindProxy());
			}

			return NewHandle;
		}

		FORCEINLINE bool operator ==(const TProxyHandle<IdType, ProxyType, ProxySettings>& InHandle) const
		{
			return InHandle.Id == Id;
		}

		FORCEINLINE TProxyHandle<IdType, ProxyType, ProxySettings>& operator =(const TProxyHandle<IdType, ProxyType, ProxySettings>& InHandle)
		{
			// 1. If local proxy valid prior to move, cache to DecRef
			ProxyType* ProxyToDecRef = nullptr;
			if (IsValid())
			{
				ProxyToDecRef = &ProxyMap->FindChecked(Id);
			}

			// 2. Move internal state
			Id = InHandle.Id;
			ProxyMap = InHandle.ProxyMap;

			// 3. IncRef new data
			if (IsValid())
			{
				ProxyType& Proxy = ProxyMap->FindChecked(Id);
				Proxy.IncRef();
			}

			// 4. DecRef post IncRef to avoid premature destruction if handling same proxy
			if (ProxyToDecRef)
			{
				ProxyToDecRef->DecRef();
			}

			return *this;
		}

		FORCEINLINE TProxyHandle<IdType, ProxyType, ProxySettings>& operator =(TProxyHandle<IdType, ProxyType, ProxySettings>&& InHandle)
		{
			// 1. If local proxy valid prior to move, cache to DecRef
			ProxyType* ProxyToDecRef = nullptr;
			if (IsValid())
			{
				ProxyToDecRef = &ProxyMap->FindChecked(Id);
			}

			// 2. Move internal state
			Id = InHandle.Id;
			ProxyMap = InHandle.ProxyMap;

			// 3. IncRef new data
			if (IsValid())
			{
				ProxyType& Proxy = ProxyMap->FindChecked(Id);
				Proxy.IncRef();
			}

			// 4. DecRef post IncRef to avoid premature destruction if handling same proxy
			if (ProxyToDecRef)
			{
				ProxyToDecRef->DecRef();
			}

			// 5. DecRef handle to move and clear state
			if (InHandle.IsValid())
			{
				ProxyType& Proxy = ProxyMap->FindChecked(InHandle.Id);
				Proxy.DecRef();
			}
			InHandle.Id = static_cast<IdType>(INDEX_NONE);
			InHandle.ProxyMap = nullptr;

			return *this;
		}
	};

	// Modulator Proxy Templates
	template <typename IdType>
	class TModulatorBase
	{
	private:
#if !UE_BUILD_SHIPPING
		FString ResolveName(const FString& InNamePath)
		{
			const FSoftObjectPath SoftObjectPath = FSoftObjectPath(InNamePath);

			if (SoftObjectPath.IsAsset())
			{
				return SoftObjectPath.GetAssetName();
			}
			else if (SoftObjectPath.IsSubobject())
			{
				FString Left;
				FString Right;
				if (SoftObjectPath.GetSubPathString().Split(".", &Left, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					return Right;
				}
			}

			return InNamePath;
		}
#endif // !UE_BUILD_SHIPPING

		IdType Id;

#if !UE_BUILD_SHIPPING
		FName Name;
		FName Path;
#endif // !UE_BUILD_SHIPPING

	public:
		TModulatorBase()
			: Id(static_cast<IdType>(INDEX_NONE))
		{
		}

		TModulatorBase(const FName InPath, const uint32 InId)
			: Id(static_cast<IdType>(InId))
#if !UE_BUILD_SHIPPING
			, Name(ResolveName(InPath.ToString()))
			, Path(InPath)
#endif // !UE_BUILD_SHIPPING
		{
		}

		TModulatorBase(const FString& InPath, const uint32 InId)
			: Id(static_cast<IdType>(InId))
#if !UE_BUILD_SHIPPING
			, Name(ResolveName(InPath))
			, Path(InPath)
#endif // !UE_BUILD_SHIPPING
		{
		}

		virtual ~TModulatorBase() = default;

		IdType GetId() const
		{
			return Id;
		}

		// FOR DEBUG USE ONLY (Not available in shipped builds):
		// Provides name of object that generated proxy.
		const FName& GetName() const
		{
#if UE_BUILD_SHIPPING
			static FName Name;
#endif // UE_BUILD_SHIPPING

			return Name;
		}

		// FOR DEBUG USE ONLY (Not available in shipped builds):
		// Provides path of object that generated proxy.
		const FName& GetPath() const
		{
#if UE_BUILD_SHIPPING
			static FName Path;
#endif // UE_BUILD_SHIPPING

			return Path;
		}
	};

	template<typename IdType, typename ProxyType, typename ProxySettings>
	class TModulatorProxyRefType : public TModulatorBase<IdType>
	{
	protected:
		uint32 RefCount;
		FAudioModulationSystem* ModSystem;

	public:
		TModulatorProxyRefType()
			: TModulatorBase<IdType>()
			, RefCount(0)
			, ModSystem(nullptr)
		{
		}

		TModulatorProxyRefType(const FName& InPath, const uint32 InId, FAudioModulationSystem& OutModSystem)
			: TModulatorBase<IdType>(InPath, InId)
			, RefCount(0)
			, ModSystem(&OutModSystem)
		{
		}

		TModulatorProxyRefType(const TModulatorProxyRefType& InProxyRef)
			: TModulatorBase<IdType>(InProxyRef.GetPath(), InProxyRef.GetId())
			, RefCount(InProxyRef.RefCount)
			, ModSystem(InProxyRef.ModSystem)
		{
		}

		TModulatorProxyRefType(TModulatorProxyRefType&& InProxyRef)
			: TModulatorBase<IdType>(InProxyRef.GetPath(), InProxyRef.GetId())
			, RefCount(InProxyRef.RefCount)
			, ModSystem(InProxyRef.ModSystem)
		{
			InProxyRef.RefCount = 0;
			InProxyRef.ModSystem = nullptr;
		}

		TModulatorProxyRefType& operator=(const TModulatorProxyRefType& InOther)
		{
			RefCount = InOther.RefCount;
			ModSystem = InOther.ModSystem;
			return *this;
		}

		TModulatorProxyRefType& operator=(TModulatorProxyRefType&& InOther)
		{
			RefCount = InOther.RefCount;
			ModSystem = InOther.ModSystem;

			InOther.RefCount = 0;
			InOther.ModSystem = nullptr;

			return *this;
		}

		virtual ~TModulatorProxyRefType()
		{
			check(RefCount == 0);
		}

		uint32 GetRefCount() const
		{
			return RefCount;
		}

	protected:
		FORCEINLINE void IncRef()
		{
			RefCount++;
		}

		FORCEINLINE uint32 DecRef()
		{
			check(RefCount != 0);

			RefCount--;
			return RefCount;
		}

	private:
		friend class TProxyHandle<IdType, ProxyType, ProxySettings>;
	};
} // namespace AudioModulation