// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationValue.h"

#include "SoundModulationProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationValue)


FSoundModulationMixValue::FSoundModulationMixValue(float InValue, float InAttackTime, float InReleaseTime)
	: TargetValue(InValue)
	, AttackTime(InAttackTime)
	, ReleaseTime(InReleaseTime)
	, Value(InValue)
	, ActiveFade(EActiveFade::Attack)
{
}

void FSoundModulationMixValue::SetActiveFade(EActiveFade InActiveFade, float InFadeTime)
{
	// Only override fades can modify existing, active behavior
	if (ActiveFade != EActiveFade::Override)
	{
		if (InActiveFade == ActiveFade)
		{
			return;
		}
	}

	ActiveFade = InActiveFade;
	switch (ActiveFade)
	{
		case EActiveFade::Attack:
		{
			if (InFadeTime > 0.0f)
			{
				AttackTime = InFadeTime;
			}
			LerpTime = AttackTime;
		}
		break;

		case EActiveFade::Release:
		{
			if (InFadeTime > 0.0f)
			{
				ReleaseTime = InFadeTime;
			}
			LerpTime = ReleaseTime;
		}
		break;
		
		case EActiveFade::Override:
		default:
		{
			if (InFadeTime > 0.0f || FMath::IsNearlyZero(InFadeTime))
			{
				LerpTime = InFadeTime;
			}
			// If fade was not set prior, use attack time as default.
			else if (LerpTime < 0.0f)
			{
				if (ensureAlwaysMsgf(AttackTime >= 0.0f, TEXT("Lerp time must be non-negative if attack time not set and overriding.")))
				{
					LerpTime = AttackTime;
				}
				else
				{
					LerpTime = 0.0f;
				}
			}
			break;
		}
	}

	UpdateDelta();
}

void FSoundModulationMixValue::SetCurrentValue(float InValue)
{
	Value = InValue;
}

float FSoundModulationMixValue::GetCurrentValue() const
{
	if (LerpTime > 0.0f)
	{
		return Value;
	}

	// Returning target when lerp is set to non-positive value
	// ensures that an update call isn't required to get the
	// current value in the same frame.
	return TargetValue;
}

void FSoundModulationMixValue::Update(double InElapsed)
{
	if (!FMath::IsNearlyEqual(LastTarget, TargetValue))
	{
		UpdateDelta();
	}

	if (Value < TargetValue)
	{
		Value = static_cast<float>(Value + (Delta * InElapsed));
		Value = FMath::Min(Value, TargetValue);
	}
	else if (Value > TargetValue)
	{
		Value = static_cast<float>(Value - (Delta * InElapsed));
		Value = FMath::Max(Value, TargetValue);
	}

}

void FSoundModulationMixValue::UpdateDelta()
{
	// Initialize to attack time if unset
	if (LerpTime < 0.0f)
	{
		if (ensureAlwaysMsgf(ActiveFade == EActiveFade::Attack, TEXT("Lerp time must be non-negative if not attacking.")))
		{
			LerpTime = AttackTime;
		}
		else
		{
			LerpTime = 0.0f;
		}
	}

	if (LerpTime > 0.0f)
	{
		Delta = FMath::Abs(Value - TargetValue) / LerpTime;
	}
	else
	{
		Delta = 0.0f;
		Value = TargetValue;
	}

	LastTarget = TargetValue;
}
