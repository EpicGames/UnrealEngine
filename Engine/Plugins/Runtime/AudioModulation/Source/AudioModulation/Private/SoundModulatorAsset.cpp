// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
#include "SoundModulatorAsset.h"
#include "Templates/SharedPointer.h"

namespace AudioModulation
{
	const FString PluginAuthor = TEXT("Epic Games, Inc.");
	const FText PluginNodeCategory = NSLOCTEXT("AudioModulation", "AudioModulationNodeCategory", "Modulation");
	const FText PluginNodeMissingPrompt = NSLOCTEXT("AudioModulation", "DefaultMissingNodePrompt", "The node was likely removed, renamed, or the AudioModulation plugin is not loaded.");

	FSoundModulatorAsset::FSoundModulatorAsset(const TSharedPtr<Audio::IProxyData>& InProxyPtr)
	{
		if (!InProxyPtr.IsValid())
		{
			return;
		}

		if (ensure(InProxyPtr->CheckTypeCast<FSoundModulatorAssetProxy>()))
		{
			FSoundModulatorAssetProxy* ModulatorProxy = static_cast<FSoundModulatorAssetProxy*>(InProxyPtr.Get());
			Proxy = MakeShared<FSoundModulatorAssetProxy, ESPMode::ThreadSafe>(*ModulatorProxy);
		}
	}

	FSoundModulationParameterAsset::FSoundModulationParameterAsset(const TSharedPtr<Audio::IProxyData>& InProxyPtr)
	{
		if (!InProxyPtr.IsValid())
		{
			return;
		}

		if (ensure(InProxyPtr->CheckTypeCast<FSoundModulationParameterAssetProxy>()))
		{
			FSoundModulationParameterAssetProxy* ParamProxy = static_cast<FSoundModulationParameterAssetProxy*>(InProxyPtr.Get());
			check(ParamProxy);

			Proxy = MakeShared<FSoundModulationParameterAssetProxy, ESPMode::ThreadSafe>(*ParamProxy);
		}
	}
} // namespace AudioModulation
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT
