// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "AudioDeviceManager.h" 
#include "AudioModulation.h"
#include "AudioModulationStatics.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "Evaluation/MovieScenePlaybackManager.h"
#include "LevelSequence.h"
#include "LevelSequencePlayer.h"
#include "Misc/AutomationTest.h"
#include "MovieScene.h"
#include "MovieSceneAudioControlBusSection.h"
#include "MovieSceneAudioControlBusTrack.h"
#include "MovieSceneFwd.h"
#include "PreviewScene.h"       
#include "SoundModulationParameter.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::MovieScene::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneControlBusSimpleTest,
		"System.Engine.AudioModulation.ControlBusTrack.SimpleParameterUpdate",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		bool FMovieSceneControlBusSimpleTest::RunTest(const FString& Parameters)
	{
		// Create a test sequence and associated control bus values
		ULevelSequence* Sequence = NewObject<ULevelSequence>(GetTransientPackage());
		UTEST_NOT_NULL(TEXT("Failed to create level sequence."), Sequence);

		Sequence->Initialize();  // Creates MovieScene subobject

		USoundModulationParameterVolume* Param = NewObject<USoundModulationParameterVolume>(
			GetTransientPackage(), NAME_None, RF_Transient);
		UTEST_NOT_NULL(TEXT("Failed to create modulation parameter."), Param);

		USoundControlBus* ControlBus = NewObject<USoundControlBus>(Sequence, TEXT("TestControlBus"), RF_Transient);
		UTEST_NOT_NULL(TEXT("Failed to create control bus."), ControlBus);

		ControlBus->Parameter = Param;

		UMovieScene* MovieScene = Sequence->GetMovieScene();
		UTEST_NOT_NULL(TEXT("Failed to get MovieScene."), MovieScene);

		ULevelSequencePlayer* Player = NewObject<ULevelSequencePlayer>(GetTransientPackage());
		UTEST_NOT_NULL(TEXT("Failed to create player."), Player);

		// Set up movie scene params to make a 10 second sequence matching existing MovieScenePlaybackManagerTests
		int32 StartTick = 0;
		int32 TickRes = 60000;
		int32 DurationTicks = TickRes * 10;
		int32 DisplayRate = 30;

		MovieScene->SetDisplayRate(FFrameRate(DisplayRate, 1));
		MovieScene->SetTickResolutionDirectly(FFrameRate(TickRes, 1));
		MovieScene->SetPlaybackRange(FFrameNumber(StartTick), DurationTicks);

		MovieScene->Modify();

		// Add new tracks and sections for playing
		UMovieSceneAudioControlBusTrack* BusTrack = Cast<UMovieSceneAudioControlBusTrack>(MovieScene->AddTrack(UMovieSceneAudioControlBusTrack::StaticClass()));
		UTEST_NOT_NULL(TEXT("Failed to add control bus track."), BusTrack);

		UMovieSceneAudioControlBusSection* Section = Cast<UMovieSceneAudioControlBusSection>(BusTrack->AddNewControlBus(ControlBus));
		UTEST_NOT_NULL(TEXT("Failed to add control bus section."), Section);

		Section->BusValue.SetDefault(0.0f);

		// Set up test world and scene to run the sequence
		FPreviewScene PreviewScene(FPreviewScene::ConstructionValues().AllowAudioPlayback(true));
		UWorld* TestWorld = PreviewScene.GetWorld();

		// Set up audio device to process modulators in this world
		FAudioDeviceParams Params = FAudioDeviceManager::Get()->GetDefaultParamsForNewWorld();
		Params.AssociatedWorld = TestWorld;
		FAudioDeviceHandle Handle = FAudioDeviceManager::Get()->RequestAudioDevice(Params);
		TestWorld->SetAudioDevice(Handle);

		// Verify the modulation system is reachable before proceeding
		AudioModulation::FAudioModulationManager* ModSystem =
			UAudioModulationStatics::GetModulation(TestWorld);
		UTEST_NOT_NULL(TEXT("ModSystem not created"), ModSystem);

		// Set up the keyframes in the section to go from max to min volume across the sequence
		TArray<FFrameNumber> FrameTimes = { FFrameNumber(0), FFrameNumber(DurationTicks) };
		TArray<FMovieSceneFloatValue> FrameValues = { FMovieSceneFloatValue(0.0f), FMovieSceneFloatValue(-60.0f) };
		Section->BusValue.Set(FrameTimes, FrameValues);

		// Init world and sequence
		Player->InitializeForTick(TestWorld);
		Player->Initialize(Sequence, TestWorld->PersistentLevel, FLevelSequenceCameraSettings{});

		FMovieSceneRootEvaluationTemplateInstance& EvalTemplate = Player->GetEvaluationTemplate();

		FRootInstanceHandle RootHandle = EvalTemplate.GetRootInstanceHandle();
		TSharedPtr<FMovieSceneEntitySystemRunner> Runner = EvalTemplate.GetRunner();

		// Init the manager to mark the sequence as playing so it evaluates
		FMovieScenePlaybackManager Manager(Sequence);
		Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

		// Handles updating the sequence to a given frame and processing it
		auto EvaluateAtDisplayFrame = [&](int32 DisplayFrame)
			{
				FMovieScenePlaybackManager::FContexts Contexts;
				Manager.UpdateTo(FFrameTime(FFrameNumber(DisplayFrame)), Contexts);
				for (const FMovieSceneContext& Ctx : Contexts)
				{
					Runner->QueueUpdate(Ctx, RootHandle);
				}
				Runner->Flush();
			};

		// Run tests for start, middle and end of sequence.
		// Evaluate the sequence to those three frame positions
		// Tell the modulation systems to process 1 second of time passing to allow the value fading to process
		// By default the MovieSceneAudioControlBusSystem uses UAudioModulationStatics::UpdateMixByFilter with negative fade time so fade is the smallest value possible
		// Query the modulation value and assert that it matches what we expect using the paramater conversion function
		
		// Evaluate sequence start
		EvaluateAtDisplayFrame(0);
		ModSystem->ProcessModulators(1); // Process modulators by 1 second deltatime to allow fading between values to finish applying
		float ActualValue = UAudioModulationStatics::GetModulatorValue(TestWorld, ControlBus);
		UTEST_NEARLY_EQUAL("Starting control bus value", ActualValue, ControlBus->Parameter->ConvertUnitToNormalized(0), UE_SMALL_NUMBER);

		// Evaluate sequence mid point
		EvaluateAtDisplayFrame(DisplayRate * 5);
		ModSystem->ProcessModulators(1);
		ActualValue = UAudioModulationStatics::GetModulatorValue(TestWorld, ControlBus);
		UTEST_NEARLY_EQUAL("Middle control bus value", ActualValue, ControlBus->Parameter->ConvertUnitToNormalized(-30), UE_SMALL_NUMBER);

		// Evaluate sequence end
		EvaluateAtDisplayFrame(DisplayRate * 10);
		ModSystem->ProcessModulators(1);
		ActualValue = UAudioModulationStatics::GetModulatorValue(TestWorld, ControlBus);
		UTEST_NEARLY_EQUAL("Ending control bus value", ActualValue, ControlBus->Parameter->ConvertUnitToNormalized(-60), UE_SMALL_NUMBER);

		// Clean up assets
		Player->Stop();

		// Tear down template
		EvalTemplate.TearDown();

		// Process modulators to clear mixes
		ModSystem->ProcessModulators(0);

		return true;
	}
}

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS