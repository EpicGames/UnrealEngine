// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDeviceManager.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"
#include "AudioModulationDestination.h"
#include "Modules/ModuleInterface.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "SoundModulationGenerator.h"
#include "SoundModulationParameter.h"
#include "Stats/Stats.h"

#define UE_API AUDIOMODULATION_API


// Cycle stats for audio mixer
DECLARE_STATS_GROUP(TEXT("AudioModulation"), STATGROUP_AudioModulation, STATCAT_Advanced);

// Tracks the time for the full render block 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Process Modulators"), STAT_AudioModulationProcessModulators, STATGROUP_AudioModulation, AUDIOMODULATION_API);

namespace AudioModulation
{
	// Forward Declarations
	class FAudioModulationSystem;

	class FAudioModulationManager : public IAudioModulationManager
	{
	public:
		UE_API FAudioModulationManager();
		UE_API virtual ~FAudioModulationManager();

		//~ Begin IAudioModulationManager implementation
		UE_API virtual void Initialize(const FAudioPluginInitializationParams& InitializationParams) override;
		UE_API virtual void OnAuditionEnd() override;

		UE_API virtual void GetControlBusAssetPath(TArray<FTopLevelAssetPath>& OutClassNames) override;

#if !UE_BUILD_SHIPPING
		UE_API virtual void SetDebugBusFilter(const FString* InFilter);
		UE_API virtual void SetDebugMixFilter(const FString* InFilter);
		UE_API virtual void SetDebugMatrixEnabled(bool bInIsEnabled);
		UE_API virtual void SetDebugGeneratorsEnabled(bool bInIsEnabled);
		UE_API virtual void SetDebugGeneratorFilter(const FString* InFilter);
		UE_API virtual void SetDebugGeneratorTypeFilter(const FString* InFilter, bool bInIsEnabled);
		UE_API virtual void SetDebugActiveMixesEnabled(bool bInIsEnabled);
		UE_API virtual void SetDebugActiveGlobalMixesEnabled(bool bInIsEnabled);

		UE_API virtual bool OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream) override;
		UE_API virtual int32 OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation) override;
		UE_API virtual bool OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream) override;
#endif // !UE_BUILD_SHIPPING

		UE_API virtual void ProcessModulators(const double InElapsed) override;
		UE_API virtual void UpdateModulator(const USoundModulatorBase& InModulator) override;

	protected:
		UE_API virtual void RegisterModulator(Audio::FModulatorHandleId InHandleId, Audio::FModulatorId InModulatorId) override;
		UE_API virtual bool GetModulatorValue(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const override;
		UE_API virtual bool GetModulatorValueThreadSafe(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const override;
		UE_API virtual void UnregisterModulator(const Audio::FModulatorHandle& InHandle) override;
	//~ End IAudioModulationManager implementation

	public:
		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestination to safety activate and track a given modulator")
		UE_API void ActivateBus(const USoundControlBus& InBus);

		UE_API void ActivateBusMix(const USoundControlBusMix& InBusMix);

		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestination to safety activate and track a given modulator")
		UE_API void ActivateGenerator(const USoundModulationGenerator& InGenerator);

		UE_API USoundControlBusMix* CreateBusMixFromValue(FName Name, const TArray<USoundControlBus*>& Buses, float Value, float AttackTime = -1.0f, float ReleaseTime = -1.0f, UObject* InOuter = nullptr);

		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestination to safety activate and track a given modulator")
		UE_API void DeactivateBus(const USoundControlBus& InBus);

		UE_API void DeactivateBusMix(const USoundControlBusMix& InBusMix);
		UE_API void DeactivateBusMixByID(const uint32 ID);
		UE_API void DeactivateAllBusMixes();

		UE_DEPRECATED(5.4, "Deactivation of modulators in this manner is now deprecated. Use USoundModulationDestination to safety activate and track a given modulator")
		UE_API void DeactivateGenerator(const USoundModulationGenerator& InGenerator);

		UE_API bool IsBusMixActive(const USoundControlBusMix& InBusMix);

		UE_API void SaveMixToProfile(const USoundControlBusMix& InBusMix, const int32 InProfileIndex);
		UE_API TArray<FSoundControlBusMixStage> LoadMixFromProfile(const int32 InProfileIndex, USoundControlBusMix& OutBusMix);

		UE_API void UpdateMix(const TArray<FSoundControlBusMixStage>& InStages, USoundControlBusMix& InOutMix, bool bInUpdateObject = false, float InFadeTime = -1.0f, double Duration = -1.0, bool bRetriggerOnActivation = false);
		UE_API void UpdateMix(const USoundControlBusMix& InMix, float InFadeTime = -1.0f);
		UE_API void UpdateMixByFilter(const FString& InAddressFilter, const TSubclassOf<USoundModulationParameter>& InParamClassFilter, USoundModulationParameter* InParamFilter, float Value, float FadeTime, USoundControlBusMix& InOutMix, bool bInUpdateObject = false);

		UE_API void SoloBusMix(const USoundControlBusMix& InBusMix);

		UE_API void SetGlobalBusMixValue(USoundControlBus& InBus, float InValue, float InFadeTime);
		UE_API void ClearGlobalBusMixValue(const USoundControlBus& InBus, float InFadeTime);
		UE_API void ClearAllGlobalBusMixValues(float InFadeTime);

		UE_API float GetModulatorValueThreadSafe(uint32 ModulationID);

		UE_API FAudioModulationSystem& GetSystem();

	private:
		FAudioModulationSystem* ModSystem = nullptr;
	};

	AUDIOMODULATION_API FAudioModulationManager* GetDeviceModulationManager(Audio::FDeviceId InDeviceId);

	AUDIOMODULATION_API void IterateModulationManagers(TFunctionRef<void(FAudioModulationManager&)> InFunction);
} // namespace AudioModulation

class FAudioModulationPluginFactory : public IAudioModulationFactory
{
public:
	virtual const FName& GetDisplayName() const override
	{
		static FName DisplayName = FName(TEXT("DefaultModulationPlugin"));
		return DisplayName;
	}

	UE_API virtual TAudioModulationPtr CreateNewModulationPlugin(FAudioDevice* OwningDevice) override;
};

class FAudioModulationModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

private:
	FAudioModulationPluginFactory ModulationPluginFactory;
};

#undef UE_API
