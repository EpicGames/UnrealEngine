// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioModulation.h"

#include "AudioThread.h"
#include "Sound/SoundModulationDestination.h"

#include "AudioModulationDestination.generated.h"

#define UE_API AUDIOMODULATION_API


UCLASS(MinimalAPI, config = Engine, editinlinenew, BlueprintType)
class UAudioModulationDestination : public UObject
{
	GENERATED_BODY()

private:
	UPROPERTY()
	TObjectPtr<const USoundModulatorBase> Modulator;

	Audio::FModulationDestination Destination;

public:
	UE_API virtual void PostInitProperties() override;

	// Returns true if a modulator was set and has been cleared.
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Clear Modulator")
	UE_API UPARAM(DisplayName = "Modulator Cleared") bool ClearModulator();

	// Returns currently set modulator.
	UFUNCTION(BlueprintPure, Category = "Audio|Modulation")
	UE_API UPARAM(DisplayName = "Modulator") const USoundModulatorBase* GetModulator() const;

	// Returns the last calculated modulator value sampled from the thread controls are processed on.
	UFUNCTION(BlueprintPure, Category = "Audio|Modulation", DisplayName = "Get Watched Modulator Value")
	UE_API UPARAM(DisplayName = "Value") float GetValue() const;

	// Returns true if modulator was set to new value or was already set to provided value.
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Set Watched Modulator")
	UE_API UPARAM(DisplayName = "Modulator Set") bool SetModulator(const USoundModulatorBase* InModulator);
};

#undef UE_API
