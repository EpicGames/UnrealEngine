// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "Misc/Build.h"
#include "SoundModulationParameter.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealType.h"

#include "AudioModulationSettings.generated.h"

#define UE_API AUDIOMODULATION_API


UCLASS(MinimalAPI, config=AudioModulation, defaultconfig, meta = (DisplayName = "Audio Modulation"))
class UAudioModulationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// Array of Modulation Parameters that are loaded on plugin startup
	UPROPERTY(config, EditAnywhere, Category = "Parameters", meta = (AllowedClasses = "/Script/AudioModulation.SoundModulationParameter"))
	TArray<FSoftObjectPath> Parameters;

	UE_API void RegisterParameters() const;

	UE_API TObjectPtr<USoundModulationParameter> GetModulationParameter(const FString& InName) const;

	virtual FName GetCategoryName() const override { return FName(TEXT("Plugins")); }

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};

#undef UE_API
