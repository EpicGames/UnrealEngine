// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Generators/SoundModulationADEnvelope.h"
#include "Generators/SoundModulationEnvelopeFollower.h"
#include "Generators/SoundModulationLFO.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "AudioModulationStatics.generated.h"

#define UE_API AUDIOMODULATION_API

// Forward Declarations
class USoundModulationWatcher;

namespace AudioModulation
{
	class FAudioModulationManager;
	class FAudioModulationSystem;
} // namespace AudioModulation


UCLASS(MinimalAPI)
class UAudioModulationStatics : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Returns world associated with provided context object
	 */
	static UE_API UWorld* GetAudioWorld(const UObject* WorldContextObject);

	/**
	 * Returns modulation implementation associated with the provided world
	 */
	static UE_API AudioModulation::FAudioModulationManager* GetModulation(UWorld* World);

	/** Manually activates a modulation bus. If called, deactivation will only occur
	 * if bus is manually deactivated or destroyed (i.e. will not deactivate
	 * when all references become inactive).
	 */
	UE_DEPRECATED(5.6, "To manually control the activation state of a modulator, use CreateModulationDestination.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Activate Control Bus", meta = (
		WorldContext = "WorldContextObject",
		DeprecatedFunction="Note", DeprecationMessage="To manually control the activation state of a modulator, use CreateModulationDestination.")
	)
	static UE_API void ActivateBus(const UObject* WorldContextObject, USoundControlBus* Bus);

	/** Manually activates a bus modulator mix. If called, deactivation will only occur
	 * if mix is manually deactivated and not referenced or destroyed (i.e. will not deactivate
	 * when all references become inactive).
	 * @param BusMix - Mix to activate
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Activate Control Bus Mix", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator")
	)
	static UE_API void ActivateBusMix(const UObject* WorldContextObject, USoundControlBusMix* Mix);

	/** Manually activates a modulation generator. If called, deactivation will only occur
	 * if generator is manually deactivated and not referenced or destroyed (i.e. will not deactivate
	 * when all references become inactive).
	 * @param Generator - Generator to activate
	 */
	UE_DEPRECATED(5.6, "To manually control the activation state of a modulator, use CreateModulationDestination.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Activate Modulation Generator", meta = (
		WorldContext = "WorldContextObject",
		DeprecatedFunction="Note", DeprecationMessage="To manually control the activation state of a modulator, use CreateModulationDestination.")
	)
	static UE_API void ActivateGenerator(const UObject* WorldContextObject, USoundModulationGenerator* Generator);

	/** Creates a modulation bus with the provided default value.
	 * @param Name - Name of bus
	 * @param Parameter - Default value for created bus
	 * @param Activate - (DEPRECATED in 5.4: Use UAudioModulationDestination) Whether or not to activate bus
	 * on creation. If true, deactivation will only occur if returned bus is manually deactivated and not referenced
	 * or destroyed (i.e. will not deactivate when all references become inactive).
	 * @return ControlBus created.  This should be stored (eg. by a Blueprint as a variable) to prevent it from being garbage collected. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create Control Bus", meta = (
		AdvancedDisplay = "3",
		WorldContext = "WorldContextObject",
		Keywords = "make modulation LPF modulator")
	)
	static UE_API UPARAM(DisplayName = "Bus") USoundControlBus* CreateBus(UObject* WorldContextObject, FName Name, USoundModulationParameter* Parameter, bool Activate = false);

	/** Create a mix with stages created for each provided bus that are initialized to the supplied value and timing parameters.
	 * @param Buses - Buses to assign stages within new mix to
	 * @param Value - Initial value for all stages created within the new mix.
	 * @param AttackTime - Fade time to user when mix activates.
	 * @param ReleaseTime - Fade time to user when mix deactivates.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", meta = (
		AdvancedDisplay = "3",
		WorldContext = "WorldContextObject",
		Keywords = "make modulation modulator stage")
	)
	static UE_API UPARAM(DisplayName = "Mix") USoundControlBusMix* CreateBusMixFromValue(
		const UObject* WorldContextObject,
		FName Name,
		const TArray<USoundControlBus*>& Buses,
		float Value = 1.0f,
		float AttackTime = 0.1f,
		float ReleaseTime = 0.1f,
		bool bActivate = true);

	/** Creates a stage used to mix a control bus.
	 * @param Bus - Bus stage is in charge of applying mix value to.
	 * @param Value - Value for added bus stage to target when mix is active.
	 * @param AttackTime - Time in seconds for stage to mix in.
	 * @param ReleaseTime - Time in seconds for stage to mix out.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create Control Bus Mix Stage", meta = (
		AdvancedDisplay = "3",
		WorldContext = "WorldContextObject",
		Keywords = "make modulation modulator stage")
	)
	static UE_API UPARAM(DisplayName = "Stage") FSoundControlBusMixStage CreateBusMixStage(
		const UObject* WorldContextObject,
		USoundControlBus* Bus,
		float Value,
		float AttackTime = 0.1f,
		float ReleaseTime = 0.1f);

	/** Creates a Control Bus Mix with the given stage configuration.
	 * @param Name - Name of mix.
	 * @param Stages - Stages mix is responsible for.
	 * @param Activate - Whether or not to activate mix on creation. If true, deactivation will only occur
	 * if returned mix is manually deactivated and not referenced or destroyed (i.e. will not deactivate
	 * when all references become inactive). It will also still deactivate based on the Duration value.
	 * @param Duration - Amount of time the Mix will stay active for before deactivating itself.
	 *					 When less than 0, the mix will stay active until manually deactivated.
	 * @param bRetriggerOnActivation - If true, if this mix is already active when Activate is called,
	 *								   stages will return to their default values before activating.
	 * @return Capture this in a Blueprint variable to prevent it from being garbage collected. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create Control Bus Mix", meta = (
		WorldContext = "WorldContextObject",
		AdvancedDisplay = "3",
		Keywords = "make modulation modulator")
	)
	static UE_API UPARAM(DisplayName = "BusMix") USoundControlBusMix* CreateBusMix(
		UObject* WorldContextObject,
		FName Name, 
		TArray<FSoundControlBusMixStage> Stages,
		bool Activate,
		double Duration = -1.0,
		bool bRetriggerOnActivation = false);

	/** Creates a modulation parameter of a given class.
	 * @param Name - Name of parameter.
	 * @param ParamClass - The type of Modulation Parameter to create.
	 * @param DefaultValue - The default normalized value of the parameter (range 0-1).
	 * @return Capture this in a Blueprint variable to prevent it from being garbage collected.
	 */
	UFUNCTION(Category = "Audio|Modulation", DisplayName = "Create Modulation Parameter", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "make modulator")
	)
	static UE_API UPARAM(DisplayName = "Parameter") USoundModulationParameter* CreateModulationParameter(
		UObject* WorldContextObject,
		FName Name,
		TSubclassOf<USoundModulationParameter> ParamClass,
		float DefaultValue);

	/** Creates a modulation generator based on an Envelope Follower with the given parameters.
	 * @param Name - Name of generator.
	 * @param Params - The Envelope Follower settings, including what data to follow.
	 * @return Capture this in a Blueprint variable to prevent it from being garbage collected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create Envelope Follower Generator", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "make modulator")
	)
	static UE_API UPARAM(DisplayName = "Generator") USoundModulationGeneratorEnvelopeFollower* CreateEnvelopeFollowerGenerator(
		UObject* WorldContextObject,
		FName Name,
		FEnvelopeFollowerGeneratorParams Params);

	/** Creates a modulation generator based on an LFO with the given parameters.
	 * @param Name - Name of generator.
	 * @param Params - The LFO Settings.
	 * @return Capture this in a Blueprint variable to prevent it from being garbage collected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create LFO Generator", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "make modulator")
	)
		static UE_API UPARAM(DisplayName = "Generator") USoundModulationGeneratorLFO* CreateLFOGenerator(
			UObject* WorldContextObject,
			FName Name,
			FSoundModulationLFOParams Params);

	/** Creates a modulation generator based on an Attack/Decay Envelope.
	 * @param Name - Name of generator.
	 * @param Params - The AD Envelope Settings.
	 * @return Capture this in a Blueprint variable to prevent it from being garbage collected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Create AD Envelope Generator", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "make modulator")
	)
	static UE_API UPARAM(DisplayName = "Generator") USoundModulationGeneratorADEnvelope* CreateADEnvelopeGenerator(
		UObject* WorldContextObject,
		FName Name,
		const FSoundModulationADEnvelopeParams& Params);

	/** Creates a modulation destination, which activates the given modulator (if not already active) 
	 * and provides a function to retrieve the last value computed of the given modulator on the modulation
	 * processing thread.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", meta = (WorldContext = "WorldContextObject", Keywords = "modulator watch watcher"))
	static UE_API UPARAM(DisplayName = "Destination") UAudioModulationDestination* CreateModulationDestination(
		UObject* WorldContextObject,
		FName Name,
		USoundModulatorBase* Modulator);

	/** Deactivates a bus. Does nothing if the provided bus is already inactive.
	 * @param Bus - Scope of modulator
	 */
	UE_DEPRECATED(5.6, "To manually control the activation state of a modulator, use CreateModulationDestination.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Deactivate Control Bus", meta = (
		WorldContext = "WorldContextObject",
		DeprecatedFunction="Note", DeprecationMessage="To manually control the activation state of a modulator, use CreateModulationDestination.")
	)
	static UE_API void DeactivateBus(const UObject* WorldContextObject, USoundControlBus* Bus);

	/** Deactivates a modulation bus mix. Does nothing if an instance of the provided bus mix is already inactive.
	 * @param Mix - Mix to deactivate
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Deactivate Control Bus Mix", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator")
	)
	static UE_API void DeactivateBusMix(const UObject* WorldContextObject, USoundControlBusMix* Mix);

	///** Deactivates a modulation bus mix. Does nothing if an instance of the provided bus mix is already inactive.
	// * @param MixID - Mix to deactivate
	// */
	static UE_API void DeactivateBusMixByID(const UObject* WorldContextObject, uint32 MixID);

	/** Deactivates a modulation generator. Does nothing if an instance of the provided generator is already inactive.
	 * @param Generator - Generator to activate
	 */
	UE_DEPRECATED(5.6, "To manually control the activation state of a modulator, use CreateModulationDestination.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Deactivate Modulation Generator (Deprecated - 5.4)", meta = (
		WorldContext = "WorldContextObject",
		DeprecatedFunction="Note", DeprecationMessage="To manually control the activation state of a modulator, use CreateModulationDestination.")
	)
	static UE_API void DeactivateGenerator(const UObject* WorldContextObject, USoundModulationGenerator* Generator);

	/** Returns whether or not a Control Bus Mix is currently active.
	 * @param Mix - the Control Bus Mix to check.
	 * @return Whether or not the Bus Mix is currently active.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Modulation", DisplayName = "Is Control Bus Mix Active", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "bus modulation modulator generator")
		)
	static UE_API UPARAM(DisplayName = "Is Active") bool IsControlBusMixActive(const UObject * WorldContextObject, USoundControlBusMix * Mix);

	/** Saves control bus mix to a profile, serialized to an ini file.  If mix is loaded, uses current proxy's state.
	 * If not, uses default UObject representation.
	 * @param Mix - Mix object to serialize to profile .ini.
	 * @param ProfileIndex - Index of profile, allowing multiple profiles can be saved for single mix object. If 0, saves to default ini profile (no suffix).
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Save Control Bus Mix to Profile", meta = (
		WorldContext = "WorldContextObject",
		AdvancedDisplay = "2",
		Keywords = "serialize modulation modulator ini")
	)
	static UE_API void SaveMixToProfile(const UObject* WorldContextObject, USoundControlBusMix* Mix, int32 ProfileIndex = 0);

	/** Loads control bus mix from a profile into UObject mix definition, deserialized from an ini file.
	 * @param Mix - Mix object to deserialize profile .ini to.
	 * @param bActivate - If true, activate mix upon loading from profile.
	 * @param ProfileIndex - Index of profile, allowing multiple profiles to be loaded to single mix object. If <= 0, loads from default profile (no suffix).
	 * @return Stages - Stage values loaded from profile (empty if profile did not exist or had no values serialized).
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Load Control Bus Mix From Profile", meta = (
		WorldContext = "WorldContextObject",
		AdvancedDisplay = "2",
		Keywords = "deserialize modulation modulator ini")
	)
	static UE_API UPARAM(DisplayName = "Stages") TArray<FSoundControlBusMixStage> LoadMixFromProfile(const UObject* WorldContextObject, USoundControlBusMix* Mix, bool bActivate = true, int32 ProfileIndex = 0);

	/** Sets a Control Bus Mix with the provided stage data, if the stages
	 *  are provided in an active instance proxy of the mix. 
	 *  Does not update UObject definition of the mix. 
	 * @param Mix - Mix to update
	 * @param Stages - Stages to set.  If stage's bus is not referenced by mix, stage's update request is ignored.
	 * @param FadeTime - Fade time to user when interpolating between current value and new values.
	 *					 If negative, falls back to last fade time set on stage. If fade time never set on stage,
	 *					 uses attack time set on stage in mix asset.
	 * @param Duration - Amount of time the Mix is activated for. When the Mix has been active for the given time,
	 *					 automatically deactivates itself. When less than 0, the duration is infinite
	 *					 (i.e. mix will stay active until manually deactivated).
	 * @param bRetriggerOnActivation - If true, if this mix is already active when Activate is called,
	 *								  stages will return to their default values before activating.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Set Control Bus Mix", meta = (
		WorldContext = "WorldContextObject",
		AdvancedDisplay = "3",
		Keywords = "modulation modulator stage")
	)
	static UE_API void UpdateMix(const UObject* WorldContextObject, USoundControlBusMix* Mix, TArray<FSoundControlBusMixStage> Stages, float FadeTime = -1.0f, double Duration = -1.0, bool bRetriggerOnActivation = false);

	/** Sets a Global Control Bus Mix with a single stage associated with the provided Bus to the given float value.  This call should
	 * be reserved for buses that are to be always active. It is *NOT* recommended for transient buses, as not calling clear can keep
	 * buses active indefinitely.
	 * @param Bus - Bus associated with mix to update
	 * @param Value - Value to set global stage to.
	 * @param FadeTime - Fade time to user when interpolating between current value and new value. If negative, falls back to last fade
	 * time set on stage. If fade time never set on stage, defaults to 100ms.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Set Global Control Bus Mix Value", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator stage")
	)
	static UE_API void SetGlobalBusMixValue(const UObject* WorldContextObject, USoundControlBus* Bus, float Value, float FadeTime = -1.0f);

	/** Clears global control bus mix if set, using the applied fade time to return to the provided bus's parameter default value.
	 * @param Bus - Bus associated with mix to update
	 * @param FadeTime - Fade time to user when interpolating between current value and new values.
	 *					 If non-positive, change is immediate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Clear Global Control Bus Mix Value", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator stage")
	)
	static UE_API void ClearGlobalBusMixValue(const UObject* WorldContextObject, USoundControlBus* Bus, float FadeTime = -1.0f);

	/** Clears all global control bus mix values if set, using the applied fade time to return all to their respective bus's parameter default value.
	 * @param FadeTime - Fade time to user when interpolating between current value and new values.
	 *					 If non-positive, change is immediate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Clear All Global Control Bus Mix Values", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator stage")
	)
	static UE_API void ClearAllGlobalBusMixValues(const UObject* WorldContextObject, float FadeTime = -1.0f);

	/** Deactivates all currently active Control Bus Mixes. This includes the Global Control Bus Mixes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Deactivate All Control Bus Mixes", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation modulator stage")
	)
	static UE_API void DeactivateAllBusMixes(const UObject* WorldContextObject);

	/** Sets filtered stages of a given class to a provided target value for active instance of mix.
	 * Does not update UObject definition of mix.
	 * @param Mix - Mix to modify
	 * @param AddressFilter - (Optional) Address filter to apply to provided mix's stages.
	 * @param ParamClassFilter - (Optional) Filters buses by parameter class.
	 * @param ParamFilter - (Optional) Filters buses by parameter.
	 * @param Value - Target value to mix filtered stages to.
	 * @param FadeTime - If non-negative, updates the fade time for the resulting bus stages found matching the provided filter.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Set Control Bus Mix By Filter", meta = (
		AdvancedDisplay = "6",
		WorldContext = "WorldContextObject",
		Keywords = "control class modulation modulator stage value")
	)
	static UE_API void UpdateMixByFilter(
		const UObject* WorldContextObject,
		USoundControlBusMix* Mix,
		FString AddressFilter,
		TSubclassOf<USoundModulationParameter> ParamClassFilter,
		USoundModulationParameter* ParamFilter,
		float Value = 1.0f,
		float FadeTime = -1.0f);

	/** Commits updates from a UObject definition of a bus mix to active instance in audio thread
	 * (ignored if mix has not been activated).
	 * @param Mix - Mix to update
	 * @param FadeTime - Fade time to user when interpolating between current value and new values.
	 *					 If negative, falls back to last fade time set on stage. If fade time never set on stage,
	 *					 uses attack time set on stage in mix asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Update Control Bus Mix", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "set modulation modulator")
	)
	static UE_API void UpdateMixFromObject(const UObject* WorldContextObject, USoundControlBusMix* Mix, float FadeTime = -1.0f);

	/** Commits updates from a UObject definition of a modulator (e.g. Bus, Bus Mix, Generator)
	 *  to active instance in audio thread (ignored if modulator type has not been activated).
	 * @param Modulator - Modulator to update
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation", DisplayName = "Update Modulator", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "set control bus mix modulation modulator generator")
	)
	static UE_API void UpdateModulator(const UObject* WorldContextObject, USoundModulatorBase* Modulator);
	
	/** Gets the (normalized) value of the given modulator. 
	 * @return Value - The current value of the modulator. If the modulator is not active, returns 1.0.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Modulation", DisplayName = "Get Modulator Value", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation generator bus")
	)
	static UE_API UPARAM(DisplayName = "Value") float GetModulatorValue(const UObject* WorldContextObject, USoundModulatorBase* Modulator);
	
	/** Gets the list of modulators currently applied to a Modulation Destination.
	* @param Destination - The Modulation Destination.
	* @return Modulators - The set of Modulators.
	*/
	UFUNCTION(BlueprintPure, Category = "Audio|Modulation", DisplayName = "Get Modulators From Destination", meta = (
		WorldContext = "WorldContextObject",
		Keywords = "modulation generator bus")
	)
	static UE_API UPARAM(DisplayName = "Modulators") TSet<USoundModulatorBase*> GetModulatorsFromDestination(const FSoundModulationDestinationSettings& Destination);
};

#undef UE_API
