// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Color.h"

#include "AudioModulationStyle.generated.h"

UCLASS(MinimalAPI)
class UAudioModulationStyle : public UBlueprintFunctionLibrary
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation|Style")
	static const FColor GetModulationGeneratorColor() { return FColor(204, 51, 153); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation|Style")
	static const FColor GetControlBusColor() { return FColor(255, 51, 153); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation|Style")
	static const FColor GetControlBusMixColor() { return FColor(255, 153, 153); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation|Style")
	static const FColor GetPatchColor() { return FColor(255, 204, 255); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Modulation|Style")
	static const FColor GetParameterColor() { return FColor(255, 102, 153); }
};
