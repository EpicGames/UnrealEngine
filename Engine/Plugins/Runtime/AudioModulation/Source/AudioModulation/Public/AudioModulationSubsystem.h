// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/AudioEngineSubsystem.h"

#include "AudioModulationSubsystem.generated.h"

#define UE_API AUDIOMODULATION_API

/**
*  UAudioModulationSubsystem
*/
UCLASS(MinimalAPI)
class UAudioModulationSubsystem : public UAudioEngineSubsystem
{
	GENERATED_BODY()

public:
	UAudioModulationSubsystem() = default;
	virtual ~UAudioModulationSubsystem() override = default;

	//~ Begin USubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	// Modulation Parameter Overrides
	void SetModulationParameterClampOverride(const FString& ParameterName, bool bClamp) const;
	void ClearModulationParameterClampOverride(const FString& ParameterName) const;
	void ClearAllModulationParameterClampOverrides() const;

private:
	TMap<FString, bool> ModulationParameterClampOverrides;
};

# undef UE_API
