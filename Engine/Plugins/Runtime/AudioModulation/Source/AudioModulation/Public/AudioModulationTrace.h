// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"

namespace Audio::Modulation::Trace::EventLog::ID
{
	extern AUDIOMODULATION_API const FString ModulatorActivated;
	extern AUDIOMODULATION_API const FString ModulatorDeactivated;
} // namespace Audio::Modulation::Trace::EventLog::ID
