// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/Envelope.h"
#include "IAudioModulation.h"
#include "SoundModulationGenerator.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SoundModulationADEnvelope.generated.h"

USTRUCT(BlueprintType)
struct FSoundModulationADEnvelopeParams
{
	GENERATED_USTRUCT_BODY()

	/** Attack time of the envelope (seconds). */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 20, EditCondition = "!bBypass", UIMin = "0", ClampMin = "0"))
	float AttackTime = 0.0f;

	/** Decay time of the envelope (seconds). */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 30, EditCondition = "!bBypass", UIMin = "0", ClampMin = "0"))
	float DecayTime = 1.0f;

	/** The exponential curve factor of the attack. 1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 40, EditCondition = "!bBypass", UIMin = "0", ClampMin = "0"))
	float AttackCurve = 1.0f;

	/** The exponential curve factor of the decay. 1.0 = linear decay, < 1.0 exponential decay, > 1.0 logarithmic decay. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 50, EditCondition = "!bBypass", UIMin = "0", ClampMin = "0"))
	float DecayCurve = 1.0f;

	/** Whether or not to loop the envelope. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 60, EditCondition = "!bBypass"))
	bool bLooping = true;

	/** If true, bypasses envelope bus from being modulated by parameters, patches, or mixed (Envelope remains active and computed). */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 10))
	bool bBypass = false;
};


UCLASS(BlueprintType, hidecategories = Object, editinlinenew, MinimalAPI)
class USoundModulationGeneratorADEnvelope : public USoundModulationGenerator
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (ShowOnlyInnerProperties))
	FSoundModulationADEnvelopeParams Params;

	virtual AudioModulation::FGeneratorPtr CreateInstance() const override;
};
