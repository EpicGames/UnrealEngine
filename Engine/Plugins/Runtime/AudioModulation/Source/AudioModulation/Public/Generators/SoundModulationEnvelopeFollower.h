// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SoundModulationGenerator.h"

#include "AudioDeviceManager.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/MultithreadedPatching.h"
#include "IAudioModulation.h"
#include "SoundModulationGenerator.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SoundModulationEnvelopeFollower.generated.h"

#define UE_API AUDIOMODULATION_API


// Forward Declarations
class UAudioBus;

USTRUCT(BlueprintType)
struct FEnvelopeFollowerGeneratorParams
{
	GENERATED_USTRUCT_BODY()

	/** If true, bypasses generator from being modulated by parameters, patches, or mixed (remains active and computed). */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite)
	bool bBypass = false;

	/** If true, inverts output */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (EditCondition = "!bBypass", DisplayAfter = "ReleaseTime"))
	bool bInvert = false;

	/** AudioBus to follow amplitude of and generate modulation control signal from. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (EditCondition = "!bBypass"))
	TObjectPtr<UAudioBus> AudioBus = nullptr;

	/** Gain to apply to amplitude signal. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (EditCondition = "!bBypass", ClampMin = 0.0f))
	float Gain = 1.0f;

	/** Attack time of envelope response (in sec) */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (EditCondition = "!bBypass", ClampMin = 0.0f))
	float AttackTime = 0.010f;

	/** Release time of envelope response (in sec) */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (EditCondition = "!bBypass", ClampMin = 0.0f))
	float ReleaseTime = 0.100f;
};

UCLASS(MinimalAPI, hidecategories = Object, BlueprintType, editinlinenew, meta = (DisplayName = "Envelope Follower Generator"))
class USoundModulationGeneratorEnvelopeFollower : public USoundModulationGenerator
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (ShowOnlyInnerProperties))
	FEnvelopeFollowerGeneratorParams Params;

	UE_API virtual AudioModulation::FGeneratorPtr CreateInstance() const override;
};

#undef UE_API
