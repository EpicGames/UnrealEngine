// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DSP/LFO.h"
#include "IAudioModulation.h"
#include "SoundModulationGenerator.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SoundModulationLFO.generated.h"


UENUM(BlueprintType)
enum class ESoundModulationLFOShape : uint8
{
	Sine			  UMETA(DisplayName = "Sine"),
	UpSaw			  UMETA(DisplayName = "Saw (Up)"),
	DownSaw			  UMETA(DisplayName = "Saw (Down)"),
	Square			  UMETA(DisplayName = "Square"),
	Triangle		  UMETA(DisplayName = "Triangle"),
	Exponential		  UMETA(DisplayName = "Exponential"),
	RandomSampleHold  UMETA(DisplayName = "Random"),

	COUNT UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSoundModulationLFOParams
{
	GENERATED_USTRUCT_BODY()

	/** Shape of oscillating waveform */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 20, EditCondition = "!bBypass"))
	ESoundModulationLFOShape Shape = ESoundModulationLFOShape::Sine;

	/** Factor increasing/decreasing curvature of exponential LFO shape. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 21, EditCondition = "!bBypass && Shape == ESoundModulationLFOShape::Exponential", EditConditionHides, ClampMin = "0.000001", UIMin = "0.25", UIMax = "10.0"))
	float ExponentialFactor = 3.5f;

	/** Pulse width of square LFO shape. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 21, EditCondition = "!bBypass && Shape == ESoundModulationLFOShape::Square", EditConditionHides, ClampMin = "0.0", ClampMax = "1.0"))
	float Width = 0.5f;

	/** Amplitude of oscillator */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 30, EditCondition = "!bBypass", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float Amplitude = 1.0f;

	/** Frequency of oscillator */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 40, EditCondition = "!bBypass", UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "20"))
	float Frequency = 1.0f;

	/** Amplitude offset of oscillator */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 50, EditCondition = "!bBypass", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float Offset = 0.0f;

	/** Unit phase offset of oscillator */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 60, EditCondition = "!bBypass", UIMin = "0", UIMax = "1", ClampMin = "0"))
	float Phase = 0.0f;

	/** Whether or not to loop the oscillation more than once */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 70, EditCondition = "!bBypass"))
	bool bLooping = true;

	/** If true, bypasses LFO bus from being modulated by parameters, patches, or mixed (LFO remains active and computed). */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayPriority = 10))
	bool bBypass = false;
};


UCLASS(BlueprintType, hidecategories = Object, editinlinenew, MinimalAPI)
class USoundModulationGeneratorLFO : public USoundModulationGenerator
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (ShowOnlyInnerProperties))
	FSoundModulationLFOParams Params;

	virtual AudioModulation::FGeneratorPtr CreateInstance() const override;
};
