// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if WITH_AUDIOMODULATION
#if !UE_BUILD_SHIPPING

#include "AudioDefines.h"
#include "Containers/Map.h"
#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"

#define UE_API AUDIOMODULATION_API

namespace AudioModulation
{
	constexpr FLazyName DebugDataModularFeatureName = "AudioModulationDebugDataProvider";
	
	struct FControlBusMixStageDebugInfo
	{
		float TargetValue;
		float CurrentValue;
	};
	
	struct FControlBusMixDebugInfo
	{
		FString Name;
		uint32 Id = -1;
		uint32 RefCount = 0;
		bool bIsGlobal = false;
		double Timer = -1.0;		

		TMap<uint32, FControlBusMixStageDebugInfo> Stages;

		bool operator==(const FControlBusMixDebugInfo& Other) const
		{
			return this->Id == Other.Id && this->Name == Other.Name;
		}
	};

	FORCEINLINE uint32 GetTypeHash(const FControlBusMixDebugInfo& DebugInfo)
	{
		return HashCombine(GetTypeHash(DebugInfo.Name), DebugInfo.Id);
	}

	struct FControlBusDebugInfo
	{
		FString Name;
		FString ParameterName;
		float DefaultValue;
		float GeneratorValue;
		float MixValue;
		float Value;
		uint32 Id;
		uint32 RefCount;

		bool operator==(const FControlBusDebugInfo& Other) const
		{
			return this->Id == Other.Id && this->Name == Other.Name;
		}
	};

	FORCEINLINE uint32 GetTypeHash(const FControlBusDebugInfo& DebugInfo)
	{
		return HashCombine(GetTypeHash(DebugInfo.Name), DebugInfo.Id);
	}


	class IDebugDataProvider : public IModularFeature
	{
	public:
		virtual Audio::FDeviceId GetAssociatedDeviceID() = 0;
		UE_API virtual void GetControlBusDebugInfo(TArray<FControlBusDebugInfo>& OutDebugInfo);
		UE_API virtual void GetControlBusMixDebugInfo(TArray<FControlBusMixDebugInfo>& OutDebugInfo);
	};
}

#undef UE_API

#endif // !UE_BUILD_SHIPPING
#endif // WITH_AUDIOMODULATION

