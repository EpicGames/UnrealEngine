// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "IAudioModulation.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "SoundControlBusMix.h"

#include "MovieSceneAudioControlBusMixSection.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneAudioControlBusMixSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	AUDIOMODULATION_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	AUDIOMODULATION_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	AUDIOMODULATION_API void SetMixBus(USoundControlBusMix* NewMixBus);

	UPROPERTY()
	FMovieSceneBoolChannel BusValue;

	UPROPERTY()
	TObjectPtr<USoundControlBusMix> MixBus;
};

