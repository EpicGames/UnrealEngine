// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "SoundControlBusMix.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneAudioControlBusMixTrack.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneAudioControlBusMixTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_UCLASS_BODY()

public:
	AUDIOMODULATION_API UMovieSceneSection* AddNewControlBusMix(USoundControlBusMix* BusMix);

	// UMovieSceneTrack interface

	AUDIOMODULATION_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	AUDIOMODULATION_API virtual void RemoveAllAnimationData() override;
	AUDIOMODULATION_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	AUDIOMODULATION_API virtual void AddSection(UMovieSceneSection& Section) override;
	AUDIOMODULATION_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	AUDIOMODULATION_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	AUDIOMODULATION_API virtual bool IsEmpty() const override;
	AUDIOMODULATION_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	AUDIOMODULATION_API virtual bool SupportsMultipleRows() const override;
	AUDIOMODULATION_API virtual UMovieSceneSection* CreateNewSection() override;

	AUDIOMODULATION_API virtual void GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const override;

#if WITH_EDITOR
	AUDIOMODULATION_API virtual FText GetDefaultDisplayName() const override;
#endif

private:

	//~ UObject interface
#if WITH_EDITOR
	AUDIOMODULATION_API virtual void PostLoad() override;
#endif

	/** List of all root mix bus sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> MixSections;
};

