// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "IAudioModulation.h"
#include "MovieSceneSection.h"
#include "SoundControlBus.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneAudioControlBusSection.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneAudioControlBusSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	AUDIOMODULATION_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	AUDIOMODULATION_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;


	AUDIOMODULATION_API void SetBus(USoundControlBus* NewBus);

	UPROPERTY()
	FMovieSceneFloatChannel BusValue;

	UPROPERTY()
	TObjectPtr<USoundControlBus> ControlBus;
};

