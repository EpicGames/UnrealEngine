// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneAudioControlBusSection.h"
#include "MovieSceneAudioControlBusMixSection.h"
#include "EntitySystem/MovieSceneEntityIDs.h"

#include "MovieSceneAudioModulationComponentTypes.generated.h"

/** Component data for control bus tracks */
USTRUCT()
struct FMovieSceneAudioControlBusComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneAudioControlBusSection> Section = nullptr;
};

/** Component data for control bus mix tracks */
USTRUCT()
struct FMovieSceneAudioControlBusMixComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneAudioControlBusMixSection> Section = nullptr;
};

struct FMovieSceneAudioModulationComponentTypes
{
public:
	AUDIOMODULATION_API ~FMovieSceneAudioModulationComponentTypes();

	static AUDIOMODULATION_API FMovieSceneAudioModulationComponentTypes* Get();
	static AUDIOMODULATION_API void Destroy();

	UE::MovieScene::TComponentTypeID<FMovieSceneAudioControlBusComponentData> AudioControlBus;
	UE::MovieScene::TComponentTypeID<FMovieSceneAudioControlBusMixComponentData> AudioControlBusMix;

private:
	FMovieSceneAudioModulationComponentTypes();
};