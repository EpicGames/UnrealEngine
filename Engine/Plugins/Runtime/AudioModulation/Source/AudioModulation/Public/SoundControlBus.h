// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioDeviceManager.h"
#include "AudioDevice.h"
#include "Containers/Array.h"
#include "IAudioModulation.h"
#include "SoundModulationGenerator.h"
#include "SoundModulationParameter.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SoundControlBus.generated.h"


// Forward Declarations
class USoundModulatorBase;

struct FPropertyChangedEvent;


UCLASS(BlueprintType, hidecategories = Object, editinlinenew, MinimalAPI)
class USoundControlBus : public USoundModulatorBase
{
	GENERATED_UCLASS_BODY()

public:
	/** If true, bypasses control bus from being modulated by parameters, patches, or mixed (control bus remains active and computed). */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadWrite)
	bool bBypass;

#if WITH_EDITORONLY_DATA
	/** If true, Address field is used in place of object name for address used when applying mix changes using filtering. */
	UPROPERTY(EditAnywhere, Category = "Advanced|Mix", BlueprintReadWrite)
	bool bOverrideAddress;
	
	UPROPERTY(EditAnywhere, Category = General, meta=(MultiLine=true))
	FString Description;
#endif // WITH_EDITORONLY_DATA

	/** Address to use when applying mix changes. */
	UPROPERTY(EditAnywhere, Category = "Advanced|Mix", BlueprintReadWrite, meta = (EditCondition = "bOverrideAddress"))
	FString Address;

	/** The Modulation Generators which modulate this Control Bus. */
	UPROPERTY(EditAnywhere, Category = "Advanced|Generators", BlueprintReadWrite)
	TArray<TObjectPtr<USoundModulationGenerator>> Generators;

	/** The Modulation Parameter to use with this Control Bus. This determines how the control bus will be mixed with other modulators as well as its unit range. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	TObjectPtr<USoundModulationParameter> Parameter;

	/* UObject Implementation */
#if WITH_EDITOR
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostInitProperties() override;
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
#endif // WITH_EDITOR

	virtual void BeginDestroy() override;

	/* USoundModulatorBase Implementation */
	virtual TUniquePtr<Audio::IModulatorSettings> CreateProxySettings() const override;
	virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	virtual const Audio::FModulationParameter& GetOutputParameter() const override;

	const Audio::FModulationMixFunction GetMixFunction() const;

	float GetDefaultNormalizedValue() const { return Parameter ? Parameter->Settings.ValueNormalized : 1.0f; }

};
