// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IAudioModulation.h"
#include "SoundModulationValue.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

#include "SoundControlBusMix.generated.h"

#define UE_API AUDIOMODULATION_API

// Forward Declarations
class USoundControlBus;


USTRUCT(BlueprintType)
struct FSoundControlBusMixStage
{
	GENERATED_USTRUCT_BODY()

	UE_API FSoundControlBusMixStage();
	UE_API FSoundControlBusMixStage(USoundControlBus* InBus, const float TargetValue);

	/* Bus controlled by the mix stage. */
	UPROPERTY(EditAnywhere, Category = Stage, BlueprintReadWrite)
	TObjectPtr<USoundControlBus> Bus;

	/* Value mix is set to. */
	UPROPERTY(EditAnywhere, Category = Stage, BlueprintReadWrite)
	FSoundModulationMixValue Value;
};

UCLASS(config = Engine, autoexpandcategories = (Stage, Mix), editinlinenew, BlueprintType, MinimalAPI, meta = (PrioritizeCategories = "Mix Config Stages"))
class USoundControlBusMix : public UObject
{
	GENERATED_UCLASS_BODY()

protected:
	// Loads the mix from the provided profile index
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "50"))
	void LoadMixFromProfile();

	// Saves the mix to the provided profile index
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "60"))
	void SaveMixToProfile();

	// Solos this mix, deactivating all others and activating this
	// (if its not already active), while testing in-editor in all
	// active worlds
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "40"))
	void SoloMix();

	// Activates this mix in all active worlds.
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "10"))
	void ActivateMix();

	// Deactivates this mix in all active worlds. The mix is fully deactivated once all stages have finished their release times.
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "20"))
	void DeactivateMix();

	// Deactivates all mixes in all active worlds
	UFUNCTION(Category = Mix, meta = (CallInEditor = "true", DisplayPriority = "30"))
	void DeactivateAllMixes();

public:
	UPROPERTY(EditAnywhere, Transient, Category = Config, meta = (DisplayPriority = "30"))
	uint32 ProfileIndex = 0;

	// Once activated, the mix will start a timer for the given duration (seconds).
	// When the timer ends, the mix will be deactivated.
	// As a result, the attack time is included in this duration, but not the release time.
	// When set to 0, the mix is activated and then immediately deactivated.
	// If less than 0, the mix will remain activated until manually deactivated.
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayPriority = "10"), BlueprintReadOnly)
	double Duration = -1.0;

	// If a Mix is already active and you activate it again, one of two things will happen:
	// If set to true, the stages will all go back to their default values and the mix will activate again,
	// allowing the attack to trigger again.
	// If set to false, calling activate will only reset the timer to deactivate (based on the Duration value).
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayPriority = "20"), BlueprintReadOnly)
	bool bRetriggerOnActivation = false;
	
	virtual void BeginDestroy() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	virtual void OnPropertyChanged(FProperty* Property, EPropertyChangeType::Type ChangeType);
#endif // WITH_EDITOR

	/* Array of stages controlled by mix. */
	UPROPERTY(EditAnywhere, Category = Stages, BlueprintReadOnly)
	TArray<FSoundControlBusMixStage> MixStages;
};

#undef UE_API
