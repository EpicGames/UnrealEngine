// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioDeviceManager.h"
#include "Containers/Queue.h"
#include "DSP/LFO.h"
#include "IAudioModulation.h"
#include "SoundModulationValue.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SoundModulationGenerator.generated.h"

#define UE_API AUDIOMODULATION_API


namespace AudioModulation
{
	class IGenerator;
	using FGeneratorPtr = TUniquePtr<IGenerator>;

	// Hacky base class to avoid 8 bytes of padding after the vtable
	class IGeneratorFixLayout
	{
	public:
		virtual ~IGeneratorFixLayout() = default;
	};

	class IGenerator : public IGeneratorFixLayout
	{
	public:
		virtual ~IGenerator() = default;

		/** Pumps commands from Audio Thread to the generator's modulation processing thread.*/
		UE_API void PumpCommands();

		// Clone the generator
		virtual FGeneratorPtr Clone() const = 0;

		/** Allows child generator class to override default copy/update behavior when receiving an updated generator call
		  * from the audio thread. Useful for ignoring updates while a generator is running or deferring the transition
		  * to the new generator state to the modulation processing thread.  This enables interpolating between existing
		  * and new generator state, properties, avoiding discontinuities, etc.
		  * @param InGenerator - The constructed version of the generator being sent from the Audio Thread
		  */
		virtual void UpdateGenerator(FGeneratorPtr&& InGenerator) = 0;

		/** Returns current value of the generator. */
		virtual float GetValue() const = 0;

		/** (Optional) Initializer step where the generator is provided the associated parent AudioDevice's Id. */
		virtual void Init(Audio::FDeviceId InDeviceId) { }

		/** Returns whether or not the generator is bypassed. */
		virtual bool IsBypassed() const = 0;

		/** Updates the generators value at the audio block rate on the modulation processing thread. */
		virtual void Update(double InElapsed) = 0;

#if !UE_BUILD_SHIPPING
		virtual void GetDebugValues(TArray<FString>& OutDebugValues) const = 0;

		virtual void GetDebugCategories(TArray<FString>& OutDebugCategories) const = 0;

		// Required for instance look-up in factory registration
		virtual const FString& GetDebugName() const = 0;
#endif // !UE_BUILD_SHIPPING

	protected:
		UE_API void AudioRenderThreadCommand(TUniqueFunction<void()>&& InCommand);

		Audio::FDeviceId AudioDeviceId = INDEX_NONE;

	private:
		TQueue<TUniqueFunction<void()>> CommandQueue;
	};
} // namespace AudioModulation

/**
 * Base class for modulators that algorithmically generate values that can effect
 * various endpoints (ex. Control Buses & Parameter Destinations)
 */
UCLASS(MinimalAPI, hideCategories = Object, abstract)
class USoundModulationGenerator : public USoundModulatorBase
{
	GENERATED_BODY()

public:
	// Create and return pointer to new instance of generator to be processed on the AudioRenderThread.
	virtual AudioModulation::FGeneratorPtr CreateInstance() const
	{
		return nullptr;
	}

	/* USoundModulatorBase Implementation */
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	UE_API virtual TUniquePtr<Audio::IModulatorSettings> CreateProxySettings() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR

	UE_API virtual void BeginDestroy() override;
};

#undef UE_API
