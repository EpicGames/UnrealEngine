// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioModulation.h"
#include "IAudioProxyInitializer.h"
#include "Math/Interval.h"
#include "Sound/SoundModulationDestination.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SoundModulationParameter.generated.h"

#define UE_API AUDIOMODULATION_API


USTRUCT(BlueprintType)
struct FSoundModulationParameterSettings
{
	GENERATED_USTRUCT_BODY()

	/** Default value of modulator (unitless) */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float ValueNormalized = 1.0f;

	/** Whether or not to clamp values using this parameter to the normalized [0,1] range. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (EditCondition = "bShowClampOption"))
	bool bClampToNormalizedRange = true;
	
#if WITH_EDITORONLY_DATA
	/** (Optional) Text name of parameter's unit */
	UPROPERTY(EditAnywhere, Category = General)
	FText UnitDisplayName;

	/**
	  * Default value of the modulator. To ensure bypass functionality of mixing, patching, and modulating
	  * functions as anticipated, value should be selected such that GetMixFunction (see USoundModulationParameter)
	  * reduces to an identity function (i.e. function acts as a "pass-through" for all values in the range [0.0, 1.0]).
	  * That is to say, this should be set to the value which has no effect on the sound.
	  */
	UPROPERTY(Transient, EditAnywhere, Category = General)
	float ValueUnit = 1.0f;

	UPROPERTY()
	bool bShowClampOption = false;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, BlueprintType)
class USoundModulationParameter : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (DisplayName = "Parameter"))
	FSoundModulationParameterSettings Settings;

public:
	/** Whether or not the parameter requires a unit conversion. */
	virtual bool RequiresUnitConversion() const
	{
		return false;
	}

	/** Function used to mix modulator units together */
	virtual Audio::FModulationMixFunction GetMixFunction() const
	{
		return Audio::FModulationParameter::GetDefaultMixFunction();
	}

	/** Function used to convert normalized, unitless value to unit value */
	virtual Audio::FModulationNormalizedConversionFunction GetUnitConversionFunction() const
	{
		return Audio::FModulationParameter::GetDefaultUnitConversionFunction();
	}

	/** Function used to convert unit value to normalized, unitless value */
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const
	{
		return Audio::FModulationParameter::GetDefaultNormalizedConversionFunction();
	}

	/** Converts normalized, unitless value [0.0f, 1.0f] to unit value. */
	virtual float ConvertNormalizedToUnit(float InNormalizedValue) const final
	{
		float UnitValue = InNormalizedValue;
		GetUnitConversionFunction()(UnitValue);
		return UnitValue;
	}

	/** Converts unit value to unitless, normalized value [0.0f, 1.0f]. */
	virtual float ConvertUnitToNormalized(float InUnitValue) const final
	{
		float NormalizedValue = InUnitValue;
		GetNormalizedConversionFunction()(NormalizedValue);
		return NormalizedValue;
	}

	/** Returns default unit value (works with and without editor loaded) */
	virtual float GetUnitDefault() const final
	{
		return ConvertNormalizedToUnit(Settings.ValueNormalized);
	}

	virtual float GetUnitMin() const
	{
		return 0.0f;
	}

	virtual float GetUnitMax() const
	{
		return 1.0f;
	}

	/** Returns the range to use when FSoundModulationParameterSettings::bClampToNormalizedRange is set to false.
	 *  If overriden, the Unit/Normalized conversion functions must be written in a way to account for this range.
	 *  Returned range is unitless/"normalized". */
	virtual TRange<float> GetNormalizedLimits() const
	{
		return TRange<float>(0.0f, 1.0f);
	}

	//~Begin IAudioProxyDataFactory Interface.
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	//~ End IAudioProxyDataFactory Interface.

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API void RefreshNormalizedValue();
	UE_API void RefreshUnitValue();
#endif // WITH_EDITOR

	UE_API Audio::FModulationParameter CreateParameter() const;

	UE_API bool IsModulationParameterClamped() const;
};

// Linearly scaled value between unit minimum and maximum.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterScaled : public USoundModulationParameter
{
	GENERATED_BODY()

public:
	/** Unit minimum of modulator. Minimum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	float UnitMin = 0.0f;

	/** Unit maximum of modulator. Maximum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	float UnitMax = 1.0f;

	virtual bool RequiresUnitConversion() const override;
	virtual Audio::FModulationUnitConversionFunction GetUnitConversionFunction() const override;
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const override;
	virtual float GetUnitMin() const override;
	virtual float GetUnitMax() const override;
};

// Modulation Parameter that scales normalized, unitless value to logarithmic frequency unit space.
UCLASS(BlueprintType, MinimalAPI, abstract)
class USoundModulationParameterFrequencyBase : public USoundModulationParameter
{
	GENERATED_BODY()

public:
	virtual bool RequiresUnitConversion() const override;
	virtual Audio::FModulationUnitConversionFunction GetUnitConversionFunction() const override;
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const override;
};

// Modulation Parameter that scales normalized, unitless value to logarithmic frequency unit space with provided minimum and maximum.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterFrequency : public USoundModulationParameterFrequencyBase
{
	GENERATED_BODY()

public:
	/** Unit minimum of modulator. Minimum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	float UnitMin = MIN_FILTER_FREQUENCY;

	/** Unit maximum of modulator. Maximum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	float UnitMax = MAX_FILTER_FREQUENCY;

	virtual float GetUnitMin() const override
	{
		return UnitMin;
	}

	virtual float GetUnitMax() const override
	{
		return UnitMax;
	}
};

// Modulation Parameter that scales normalized, unitless value to logarithmic frequency unit space with standard filter min and max frequency set.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterFilterFrequency : public USoundModulationParameterFrequencyBase
{
	GENERATED_BODY()

public:
	virtual float GetUnitMin() const override
	{
		return MIN_FILTER_FREQUENCY;
	}

	virtual float GetUnitMax() const override
	{
		return MAX_FILTER_FREQUENCY;
	}
};

// Modulation Parameter that scales normalized, unitless value to logarithmic frequency unit space with standard filter min and max frequency set.
// Mixes by taking the minimum (i.e. aggressive) filter frequency of all active modulators.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterLPFFrequency : public USoundModulationParameterFilterFrequency
{
	GENERATED_BODY()

public:
	virtual Audio::FModulationMixFunction GetMixFunction() const override;
	static Audio::FModulationParameter CreateDefaultParameter();
};

// Modulation Parameter that scales normalized, unitless value to logarithmic frequency unit space with standard filter min and max frequency set.
// Mixes by taking the maximum (i.e. aggressive) filter frequency of all active modulators.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterHPFFrequency : public USoundModulationParameterFilterFrequency
{
	GENERATED_UCLASS_BODY()

public:
	virtual Audio::FModulationMixFunction GetMixFunction() const override;
	static Audio::FModulationParameter CreateDefaultParameter();
};

// Modulation Parameter that scales normalized, unitless value to bipolar range. Mixes additively.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterBipolar : public USoundModulationParameter
{
	GENERATED_UCLASS_BODY()

public:
	/** Unit range of modulator. Range is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (ClampMin = 0.00000001))
	float UnitRange = 2.0f;

	virtual bool RequiresUnitConversion() const override;
	virtual Audio::FModulationMixFunction GetMixFunction() const override;
	virtual Audio::FModulationUnitConversionFunction GetUnitConversionFunction() const override;
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const override;
	virtual float GetUnitMax() const override;
	virtual float GetUnitMin() const override;
	static Audio::FModulationParameter CreateDefaultParameter(float UnitRange = 2.0f);
};

UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterVolume : public USoundModulationParameter
{
	GENERATED_UCLASS_BODY()

public:
	/** Minimum volume of parameter (dB). Only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (ClampMax = 0.0))
	float MinVolume = -100.0f;

	/** What the maximum volume (in dB) should be if "Clamp to Normalized Range" is set to false. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly)
	float VolumeLimit = 20.0f;
	
	virtual bool RequiresUnitConversion() const override;
	virtual Audio::FModulationUnitConversionFunction GetUnitConversionFunction() const override;
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const override;
	virtual float GetUnitMin() const override;
	virtual float GetUnitMax() const override;
	virtual TRange<float> GetNormalizedLimits() const override;
	static Audio::FModulationParameter CreateDefaultParameter(float MinUnitVolume = -100.0f);

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR
};

// Modulation Parameter whose values are mixed via addition.
UCLASS(BlueprintType, MinimalAPI)
class USoundModulationParameterAdditive : public USoundModulationParameter
{
	GENERATED_UCLASS_BODY()

public:

	/** Unit minimum of modulator. Minimum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (ClampMin = "0.0"))
	float UnitMin = 0.0f;

	/** Unit maximum of modulator. Maximum is only enforced at modulation destination. */
	UPROPERTY(EditAnywhere, Category = General, BlueprintReadOnly, meta = (ClampMin = "0.0"))
	float UnitMax = 1.0f;

	virtual bool RequiresUnitConversion() const override;
	virtual Audio::FModulationMixFunction GetMixFunction() const override;
	virtual Audio::FModulationUnitConversionFunction GetUnitConversionFunction() const override;
	virtual Audio::FModulationNormalizedConversionFunction GetNormalizedConversionFunction() const override;
	virtual float GetUnitMax() const override;
	virtual float GetUnitMin() const override;
};

namespace AudioModulation
{
	class FSoundModulationPluginParameterAssetProxy : public FSoundModulationParameterAssetProxy
	{
	public:
		explicit FSoundModulationPluginParameterAssetProxy(USoundModulationParameter* InParameter);
		FSoundModulationPluginParameterAssetProxy(const FSoundModulationPluginParameterAssetProxy& InProxy) = default;
	};

	/*
	 * Returns given registered parameter instance reference or creates it from the given asset if not registered.
	 * @param InParameter - Parameter asset associated with the pre-existing or to-create parameter
	 * @param InName	  - Name of the modulator requesting the parameter.
	 * @param InClassName - Name of the modulator class requesting the parameter.
	 * 
	 */
	AUDIOMODULATION_API const Audio::FModulationParameter& GetOrRegisterParameter(const USoundModulationParameter* InParameter, const FString& InName, const FString& InClassName);
} // namespace AudioModulation

#undef UE_API
