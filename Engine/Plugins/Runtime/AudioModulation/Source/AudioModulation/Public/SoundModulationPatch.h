// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioModulation.h"
#include "SoundModulationParameter.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "WaveTableTransform.h"

#include "SoundModulationPatch.generated.h"

#define UE_API AUDIOMODULATION_API

// Forward Declarations
class USoundControlBus;


PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType)
struct FSoundModulationTransform : public FWaveTableTransform
{
	GENERATED_USTRUCT_BODY()

	virtual ~FSoundModulationTransform() override = default;
	virtual bool ShouldClampToNormalizedRange() const override { return false; }
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS


USTRUCT(BlueprintType)
struct FSoundControlModulationInput
{
	GENERATED_USTRUCT_BODY()

	UE_API FSoundControlModulationInput();

	/** Get the modulated input value on parent patch initialization and hold that value for its lifetime */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Input, meta = (DisplayName = "Sample-And-Hold", DisplayPriority = "30"))
	uint8 bSampleAndHold : 1;

	/** Transform to apply to the input prior to mix phase */
	UPROPERTY(EditAnywhere, Category = Input, meta = (DisplayPriority = "20"))
	FSoundModulationTransform Transform;

	/** The input bus */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Input, meta = (DisplayPriority = "10"))
	TObjectPtr<USoundControlBus> Bus = nullptr;

	UE_API const USoundControlBus* GetBus() const;
	UE_API const USoundControlBus& GetBusChecked() const;
};

USTRUCT(BlueprintType)
struct FSoundControlModulationPatch
{
	GENERATED_USTRUCT_BODY()

	/** Whether or not patch is bypassed (patch is still active, but always returns output parameter default value when modulated) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Inputs)
	bool bBypass = true;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Output, meta = (DisplayName = "Parameter"))
	TObjectPtr<USoundModulationParameter> OutputParameter = nullptr;

	/** Modulation inputs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Inputs)
	TArray<FSoundControlModulationInput> Inputs;
};

UCLASS(MinimalAPI, config = Engine, editinlinenew, BlueprintType)
class USoundModulationPatch : public USoundModulatorBase
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modulation, meta = (ShowOnlyInnerProperties))
	FSoundControlModulationPatch PatchSettings;

	/* USoundModulatorBase Implementation */
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	UE_API virtual const Audio::FModulationParameter& GetOutputParameter() const override;

	UE_API virtual TUniquePtr<Audio::IModulatorSettings> CreateProxySettings() const override;

#if WITH_EDITORONLY_DATA
	UE_API virtual void Serialize(FArchive& Ar) override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR
};

#undef UE_API
