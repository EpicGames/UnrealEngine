// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioModulation.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SoundModulationValue.generated.h"

#define UE_API AUDIOMODULATION_API

USTRUCT(BlueprintType)
struct FSoundModulationMixValue
{
	GENERATED_USTRUCT_BODY()

	FSoundModulationMixValue() = default;
	UE_API FSoundModulationMixValue(float InValue, float InAttackTime, float InReleaseTime);

	enum class EActiveFade : uint8
	{
		/** Value interpolating from the parameter's default value to the mix value. */
		Attack,

		/** Value interpolating from the mix value to the parameter's default value. */
		Release,

		/** User-requested fade time to an active mix by filter (ex. from Blueprint) or editor property adjustment */
		Override
	};

	/** Target value of the modulator. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayName = "Value"))
	float TargetValue = 1.0f;

#if WITH_EDITORONLY_DATA
	/** Target value of the modulator (in units if provided). */
	UPROPERTY(Transient, EditAnywhere, Category = General)
	float TargetUnitValue = 1.0f;
#endif // WITH_EDITORONLY_DATA

	/** Time it takes (in sec) to interpolate from the parameter's default value to the mix value. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayName = "Attack Time (sec)", ClampMin = "0.0", UIMin = "0.0"))
	float AttackTime = 0.1f;

	/** Time it takes (in sec) to interpolate from the current mix value to the parameter's default value. */
	UPROPERTY(EditAnywhere, Category = Modulation, BlueprintReadWrite, meta = (DisplayName = "Release Time (sec)", ClampMin = "0.0", UIMin = "0.0"))
	float ReleaseTime = 0.1f;

	UE_API void SetActiveFade(EActiveFade InActiveFade, float InFadeTime = -1.0f);

	/** Set current value (for resetting value state only as circumvents lerp, and may result in discontinuity). */
	UE_API void SetCurrentValue(float InValue);

	/** Current value lerping toward target */
	UE_API float GetCurrentValue() const;

	UE_API void Update(double Elapsed);

private:
	UE_API void UpdateDelta();

	float LerpTime = -1.0f;
	float Value = 1.0f;
	float LastTarget = -1.0f;
	float Delta = 0.0f;
	EActiveFade ActiveFade = EActiveFade::Attack;
};

#undef UE_API
