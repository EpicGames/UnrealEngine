// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
#include "IAudioModulation.h"
#include "IAudioProxyInitializer.h"
#include "Internationalization/Text.h"

#include "MetasoundDataReferenceMacro.h"

namespace AudioModulation
{
	extern const FString AUDIOMODULATION_API PluginAuthor;
	extern const FText AUDIOMODULATION_API PluginNodeCategory;
	extern const FText AUDIOMODULATION_API PluginNodeMissingPrompt;

	class FSoundModulatorAsset
	{
		FSoundModulatorAssetProxyPtr Proxy;

	public:
		FSoundModulatorAsset() = default;

		FSoundModulatorAsset(const FSoundModulatorAsset& InOther)
			: Proxy(InOther.Proxy)
		{
		}

		FSoundModulatorAsset& operator=(const FSoundModulatorAsset& InOther)
		{
			Proxy = InOther.Proxy;
			return *this;
		}

		FSoundModulatorAsset& operator=(FSoundModulatorAsset&& InOther)
		{
			Proxy = MoveTemp(InOther.Proxy);
			return *this;
		}

		AUDIOMODULATION_API FSoundModulatorAsset(const TSharedPtr<Audio::IProxyData>& InInitData);

		Audio::FModulatorId GetModulatorId() const
		{
			if (Proxy.IsValid())
			{
				return Proxy->GetModulatorId();
			}

			return INDEX_NONE;
		}

		bool IsValid() const
		{
			return Proxy.IsValid();
		}

		const FSoundModulatorAssetProxyPtr& GetProxy() const
		{
			return Proxy;
		}

		const FSoundModulatorAssetProxy* operator->() const
		{
			return Proxy.Get();
		}

		FSoundModulatorAssetProxy* operator->()
		{
			return Proxy.Get();
		}
				
		friend inline uint32 GetTypeHash(const AudioModulation::FSoundModulatorAsset& InModulatorAsset)
		{
			if (InModulatorAsset.IsValid())
			{
				return InModulatorAsset->GetModulatorId();
			}
			return INDEX_NONE;
		}
	};

	class FSoundModulationParameterAsset
	{
		FSoundModulationParameterAssetProxyPtr Proxy;

	public:
		FSoundModulationParameterAsset() = default;
		FSoundModulationParameterAsset(const FSoundModulationParameterAsset& InOther) = default;
		FSoundModulationParameterAsset(FSoundModulationParameterAsset&& InOther) = default;
		AUDIOMODULATION_API FSoundModulationParameterAsset(const TSharedPtr<Audio::IProxyData>& InInitData);

		FSoundModulationParameterAsset& operator=(const FSoundModulationParameterAsset& InOther) = default;
		FSoundModulationParameterAsset& operator=(FSoundModulationParameterAsset && InOther) = default;

		bool IsValid() const
		{
			return Proxy.IsValid();
		}

		const FSoundModulationParameterAssetProxyPtr& GetProxy() const
		{
			return Proxy;
		}

		const FSoundModulationParameterAssetProxy* operator->() const
		{
			return Proxy.Get();
		}

		FSoundModulationParameterAssetProxy* operator->()
		{
			return Proxy.Get();
		}

		friend inline uint32 GetTypeHash(const AudioModulation::FSoundModulationParameterAsset& InModulationParameterAsset)
		{
			if (InModulationParameterAsset.IsValid())
			{
				return GetTypeHash(InModulationParameterAsset->GetParameter());
			}
			return INDEX_NONE;
		}
	};
} // namespace AudioModulation

DECLARE_METASOUND_DATA_REFERENCE_TYPES(AudioModulation::FSoundModulatorAsset, AUDIOMODULATION_API, FSoundModulatorAssetTypeInfo, FSoundModulatorAssetReadRef, FSoundModulatorAssetWriteRef)
DECLARE_METASOUND_DATA_REFERENCE_TYPES(AudioModulation::FSoundModulationParameterAsset, AUDIOMODULATION_API, FSoundModulationParameterAssetTypeInfo, FSoundModulationParameterAssetReadRef, FSoundModulationParameterAssetWriteRef)
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT
