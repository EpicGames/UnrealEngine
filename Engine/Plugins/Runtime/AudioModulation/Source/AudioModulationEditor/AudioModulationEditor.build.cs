// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioModulationEditor : ModuleRules
{
	// Set this to false & disable MetaSound plugin dependency
	// by setting MetaSound's field '"Enabled": false' in the
	// AudioModulation.uplugin if running Modulation without
	// MetaSound support.
	public static bool bIncludeMetaSoundSupport = true;

	public AudioModulationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"Engine",
				"EditorFramework",
				"GameProjectGeneration",
				"UnrealEd",
				"PropertyEditor",
				"SequenceRecorder",
				"Slate",
				"SlateCore",
				"InputCore",				
				"AudioEditor",
				"AudioExtensions",
				"AudioModulation",
				"CurveEditor",
				"EditorWidgets",
				"ToolWidgets",
				"WaveTable",
				"WaveTableEditor",
				"MovieScene",
				"Sequencer",
				"SequencerCore",
				"MovieSceneTools",
				"LevelSequence"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Sequencer",
				"MovieSceneTools"
			}
		);

		if (bIncludeMetaSoundSupport)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MetasoundGraphCore",
					"MetasoundFrontend",
					"MetasoundEditor"
				}
			);
		}
	}
}
