// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioModulationStyle.h"
#include "SoundControlBus.h"
#include "AssetDefinition_SoundControlBus.generated.h"

UCLASS()
class UAssetDefinition_SoundControlBus : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBus", "Control Bus"); }
	virtual FLinearColor GetAssetColor() const override { return UAudioModulationStyle::GetControlBusColor(); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundControlBus::StaticClass(); }
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override
	{
		if (const USoundControlBus* Bus = Cast<USoundControlBus>(AssetData.GetAsset()))
		{
			return FText::FromString(Bus->Description);
		}
		return FText::GetEmpty();
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBusSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBusSubMenuSection", "Audio Modulation"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
