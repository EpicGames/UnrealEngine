// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioModulationStyle.h"
#include "SoundControlBusMix.h"
#include "AssetDefinition_SoundControlBusMix.generated.h"

UCLASS()
class UAssetDefinition_SoundControlBusMix : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBusMix", "Control Bus Mix"); }
	virtual FLinearColor GetAssetColor() const override { return UAudioModulationStyle::GetControlBusMixColor(); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundControlBusMix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBusMixSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundControlBusMixSubMenuSection", "Audio Modulation"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
