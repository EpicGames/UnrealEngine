// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioModulationStyle.h"
#include "SoundModulationGenerator.h"
#include "AssetDefinition_SoundModulationGenerator.generated.h"

UCLASS()
class UAssetDefinition_SoundModulationGenerator : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationGenerator", "Modulation Generator"); }
	virtual FLinearColor GetAssetColor() const override { return UAudioModulationStyle::GetModulationGeneratorColor(); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundModulationGenerator::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationGeneratorSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationGeneratorSubMenuSection", "Audio Modulation"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
