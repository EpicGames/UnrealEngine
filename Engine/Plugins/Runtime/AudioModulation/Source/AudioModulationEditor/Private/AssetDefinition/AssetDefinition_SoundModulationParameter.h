// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioModulationStyle.h"
#include "SoundModulationParameter.h"
#include "AssetDefinition_SoundModulationParameter.generated.h"

UCLASS()
class UAssetDefinition_SoundModulationParameter : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationParameter", "Modulation Parameter"); }
	virtual FLinearColor GetAssetColor() const override { return UAudioModulationStyle::GetParameterColor(); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundModulationParameter::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationParameterSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationParameterSubMenuSection", "Audio Modulation"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
