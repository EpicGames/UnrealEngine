// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SoundModulationPatch.h"

#include "Editors/ModulationPatchEditor.h"

EAssetCommandResult UAssetDefinition_SoundModulationPatch::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (UObject* Object : Objects)
	{
		if (USoundModulationPatch* Patch = Cast<USoundModulationPatch>(Object))
		{
			const TSharedRef<FModulationPatchEditor> PatchEditor = MakeShared<FModulationPatchEditor>();
			PatchEditor->Init(Mode, OpenArgs.ToolkitHost, Patch);
		}
	}
	return EAssetCommandResult::Handled;
}
