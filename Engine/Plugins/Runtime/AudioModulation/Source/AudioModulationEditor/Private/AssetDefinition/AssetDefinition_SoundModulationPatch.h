// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioModulationStyle.h"
#include "SoundModulationPatch.h"
#include "AssetDefinition_SoundModulationPatch.generated.h"

UCLASS()
class UAssetDefinition_SoundModulationPatch : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationPatch", "Parameter Patch"); }
	virtual FLinearColor GetAssetColor() const override { return UAudioModulationStyle::GetPatchColor(); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundModulationPatch::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationPatchSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundModulationPatchSubMenuSection", "Audio Modulation"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
