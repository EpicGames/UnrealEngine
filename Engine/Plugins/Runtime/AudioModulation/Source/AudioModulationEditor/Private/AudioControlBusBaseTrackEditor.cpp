// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioControlBusBaseTrackEditor.h"

#include "AssetRegistry/AssetData.h"

static bool bUseControlBusTracks = true;

static FAutoConsoleVariableRef CVarControlBusTracks(
	TEXT("au.ControlBusTracks"),
	bUseControlBusTracks,
	TEXT("Enable the control bus track and control bus mix track in sequencer using the audio modulation plugin. Default enables adding new control bus tracks\n")
	TEXT("0: Disable control bus tracks\n")
	TEXT("1: Enable control bus tracks\n"),
	ECVF_Default
);

FAudioControlBusBaseTrackEditor::FAudioControlBusBaseTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
	CVarUseControlBusTracks = IConsoleManager::Get().FindConsoleVariable(TEXT("au.ControlBusTracks"));
}

FAudioControlBusBaseTrackEditor::~FAudioControlBusBaseTrackEditor()
{
}

bool FAudioControlBusBaseTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return true;
}

bool FAudioControlBusBaseTrackEditor::IsResizable(UMovieSceneTrack* InTrack) const
{
	return false;
}

void FAudioControlBusBaseTrackEditor::AddControlBusAssetTrackToSequence(const FAssetData& InAssetData)
{
}

void FAudioControlBusBaseTrackEditor::OnAudioControlBusAssetEnterPressed(const TArray<FAssetData>& AssetData)
{
	// By default enter will select the first list element if there is at least 1 element
	if (AssetData.Num() > 0)
	{
		AddControlBusAssetTrackToSequence(AssetData[0]);
	}
}
