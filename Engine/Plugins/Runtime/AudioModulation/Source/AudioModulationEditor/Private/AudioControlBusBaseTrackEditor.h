// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ISequencer.h"
#include "MovieSceneTrackEditor.h"

#define UE_API AUDIOMODULATIONEDITOR_API

class FAudioControlBusBaseTrackEditor
	: public FMovieSceneTrackEditor
{
public:
	/**
	* Constructor
	*
	* @param InSequencer The sequencer instance to be used by this tool
	*/
	UE_API FAudioControlBusBaseTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FAudioControlBusBaseTrackEditor();

	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual bool IsResizable(UMovieSceneTrack* InTrack) const override;
	/** Audio control bus asset enter pressed */
	UE_API void OnAudioControlBusAssetEnterPressed(const TArray<FAssetData>& AssetData);

protected:
	/** Add new control bus Asset track to sequence */
	UE_API virtual void AddControlBusAssetTrackToSequence(const FAssetData& InAssetData);

	IConsoleVariable* CVarUseControlBusTracks;
};

#undef UE_API

