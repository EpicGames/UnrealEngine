// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioControlBusMixTrackEditor.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "LevelSequence.h"
#include "MovieSceneAudioControlBusMixTrack.h"
#include "SequencerSettings.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FAudioControlBusMixTrackEditor"

FAudioControlBusMixTrackEditor::FAudioControlBusMixTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FAudioControlBusBaseTrackEditor(InSequencer)
{
}

FAudioControlBusMixTrackEditor::~FAudioControlBusMixTrackEditor()
{
}

TSharedRef<ISequencerTrackEditor> FAudioControlBusMixTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShareable(new FAudioControlBusMixTrackEditor(OwningSequencer));
}

FText FAudioControlBusMixTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AudioControlBusMixTrackEditor_DisplayName", "Control Bus Mix");
}

void FAudioControlBusMixTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return;
	}

	auto SubMenuCallback = [this](FMenuBuilder& SubMenuBuilder)
		{
			TSharedPtr<ISequencer> Sequencer = GetSequencer();
			UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;

			FAssetPickerConfig AssetPickerConfig;
			{
				AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FAudioControlBusMixTrackEditor::AddControlBusAssetTrackToSequence);
				AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FAudioControlBusBaseTrackEditor::OnAudioControlBusAssetEnterPressed);
				AssetPickerConfig.bAllowNullSelection = false;
				AssetPickerConfig.bAddFilterUI = true;
				AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
				AssetPickerConfig.Filter.bRecursiveClasses = true;
				AssetPickerConfig.Filter.ClassPaths.Add(USoundControlBusMix::StaticClass()->GetClassPathName());
				AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
				AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
			const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

			TSharedRef<SWidget> ListWidget = SNew(SBox)
				.WidthOverride(WidthOverride)
				.HeightOverride(HeightOverride)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				];

			SubMenuBuilder.AddWidget(ListWidget, FText::GetEmpty(), true);
		};

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddTrack", "Control Bus Mix Track"),
		LOCTEXT("AddTooltip", "Adds a new control bus mix track that can toggle a mix on or off"),
		FNewMenuDelegate::CreateLambda(SubMenuCallback),
		false,
		FSlateIconFinder::FindIconForClass(USoundControlBusMix::StaticClass())
	);
}

bool FAudioControlBusMixTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneAudioControlBusMixTrack::StaticClass();
}

bool FAudioControlBusMixTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	if (InSequence && InSequence->IsTrackSupported(UMovieSceneAudioControlBusMixTrack::StaticClass()) == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence && InSequence->IsA(ULevelSequence::StaticClass());
}

void FAudioControlBusMixTrackEditor::AddControlBusAssetTrackToSequence(const FAssetData& InAssetData)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return;
	}

	FSlateApplication::Get().DismissAllMenus();

	USoundControlBusMix* ControlBusMix = Cast<USoundControlBusMix>(InAssetData.GetAsset());
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (!ControlBusMix || !MovieScene)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddAudioControlBusMixTrack_Transaction", "Add Control Bus Mix Track"));

	MovieScene->Modify();
	UMovieSceneAudioControlBusMixTrack* NewTrack = MovieScene->AddTrack<UMovieSceneAudioControlBusMixTrack>();

	if (NewTrack == nullptr)
	{
		UE_LOGF(LogMovieScene, Warning, "Control Bus Mix Track: New track could not be added for sequence %ls", *MovieScene->GetName());
		return;
	}

	UMovieSceneSection* NewSection = NewTrack->AddNewControlBusMix(ControlBusMix);
	check(NewSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

bool FAudioControlBusMixTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return false;
	}

	USoundControlBusMix* ControlBus = Cast<USoundControlBusMix>(Asset);
	if (ControlBus)
	{
		AddControlBusAssetTrackToSequence(FAssetData(ControlBus));
	}

	return ControlBus != nullptr;
}

const FSlateBrush* FAudioControlBusMixTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(USoundControlBusMix::StaticClass()).GetIcon();
}

#undef LOCTEXT_NAMESPACE