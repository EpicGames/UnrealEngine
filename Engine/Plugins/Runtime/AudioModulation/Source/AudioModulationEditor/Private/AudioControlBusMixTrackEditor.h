// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioControlBusBaseTrackEditor.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"

#define UE_API AUDIOMODULATIONEDITOR_API

class FAudioControlBusMixTrackEditor
	: public FAudioControlBusBaseTrackEditor
{
public:
	/**
	* Constructor
	*
	* @param InSequencer The sequencer instance to be used by this tool
	*/
	UE_API FAudioControlBusMixTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FAudioControlBusMixTrackEditor();

	/**
	 * Creates an instance of this class.  Called by a sequencer
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

private:
	/** Add new control bus mix track to sequence */
	UE_API void AddControlBusAssetTrackToSequence(const FAssetData& InAssetData) override;

};

#undef UE_API