// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioControlBusTrackEditor.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "LevelSequence.h"
#include "MovieSceneAudioControlBusTrack.h"
#include "SequencerSettings.h"
#include "SoundControlBus.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FAudioControlBusTrackEditor"

FAudioControlBusTrackEditor::FAudioControlBusTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FAudioControlBusBaseTrackEditor(InSequencer)
{
}

FAudioControlBusTrackEditor::~FAudioControlBusTrackEditor()
{
}

TSharedRef<ISequencerTrackEditor> FAudioControlBusTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShareable(new FAudioControlBusTrackEditor(OwningSequencer));
}

FText FAudioControlBusTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AudioControlBusTrackEditor_DisplayName", "Control Bus");
}

void FAudioControlBusTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return;
	}

	auto SubMenuCallback = [this](FMenuBuilder& SubMenuBuilder)
		{
			TSharedPtr<ISequencer> Sequencer = GetSequencer();
			UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;

			FAssetPickerConfig AssetPickerConfig;
			{
				AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FAudioControlBusTrackEditor::AddControlBusAssetTrackToSequence);
				AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FAudioControlBusBaseTrackEditor::OnAudioControlBusAssetEnterPressed);
				AssetPickerConfig.bAllowNullSelection = false;
				AssetPickerConfig.bAddFilterUI = true;
				AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
				AssetPickerConfig.Filter.bRecursiveClasses = true;
				AssetPickerConfig.Filter.ClassPaths.Add(USoundControlBus::StaticClass()->GetClassPathName());
				AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
				AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
			const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

			TSharedRef<SWidget> ListWidget = SNew(SBox)
				.WidthOverride(WidthOverride)
				.HeightOverride(HeightOverride)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				];

			SubMenuBuilder.AddWidget(ListWidget, FText::GetEmpty(), true);
		};

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddTrack", "Control Bus Track"),
		LOCTEXT("AddTooltip", "Adds a new control track that can be used for mixing"),
		FNewMenuDelegate::CreateLambda(SubMenuCallback),
		false,
		FSlateIconFinder::FindIconForClass(USoundControlBus::StaticClass())
	);
}

bool FAudioControlBusTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneAudioControlBusTrack::StaticClass();
}

bool FAudioControlBusTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	if (InSequence && InSequence->IsTrackSupported(UMovieSceneAudioControlBusTrack::StaticClass()) == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence && InSequence->IsA(ULevelSequence::StaticClass());
}

const FSlateBrush* FAudioControlBusTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(USoundControlBus::StaticClass()).GetIcon();
}

bool FAudioControlBusTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return false;
	}

	USoundControlBus* ControlBus = Cast<USoundControlBus>(Asset);
	if (ControlBus)
	{
		AddControlBusAssetTrackToSequence(FAssetData(ControlBus));
	}

	return ControlBus != nullptr;
}

void FAudioControlBusTrackEditor::AddControlBusAssetTrackToSequence(const FAssetData& InAssetData)
{
	if (!CVarUseControlBusTracks || !CVarUseControlBusTracks->GetBool())
	{
		return;
	}

	FSlateApplication::Get().DismissAllMenus();

	USoundControlBus* ControlBus = Cast<USoundControlBus>(InAssetData.GetAsset());
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (!ControlBus || !MovieScene)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddAudioControlBusTrack_Transaction", "Add Control Bus Track"));

	MovieScene->Modify();
	UMovieSceneAudioControlBusTrack* NewTrack = MovieScene->AddTrack<UMovieSceneAudioControlBusTrack>();

	if (NewTrack == nullptr)
	{
		UE_LOGF(LogMovieScene, Warning, "Control Bus Track: New track could not be added for sequence %ls", *MovieScene->GetName());
		return;
	}

	UMovieSceneSection* NewSection = NewTrack->AddNewControlBus(ControlBus);
	check(NewSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

#undef LOCTEXT_NAMESPACE
