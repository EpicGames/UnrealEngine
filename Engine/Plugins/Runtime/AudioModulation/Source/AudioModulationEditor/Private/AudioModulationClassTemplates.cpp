// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationClassTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioModulationClassTemplates)

USoundModulationClassTemplate::USoundModulationClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UPluginClassTemplate(ObjectInitializer)
{
	PluginName = TEXT("AudioModulation");
}

USoundModulationGeneratorClassTemplate::USoundModulationGeneratorClassTemplate(const FObjectInitializer& ObjectInitializer)
	: USoundModulationClassTemplate(ObjectInitializer)
{
	SetGeneratedBaseClass(USoundModulationGenerator::StaticClass());
}

