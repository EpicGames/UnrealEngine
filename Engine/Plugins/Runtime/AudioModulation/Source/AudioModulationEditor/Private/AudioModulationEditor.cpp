// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AudioModulation.h"
#include "Editors/ModulationPatchCurveEditorViewStacked.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IAudioModulation.h"
#include "ICurveEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Layouts/SoundControlBusMixStageLayout.h"
#include "Layouts/SoundControlModulationPatchLayout.h"
#include "Layouts/SoundModulationParameterSettingsLayout.h"
#include "Layouts/SoundModulationTransformLayout.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneToolsModule.h"
#include "Sound/SoundBase.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatch.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/UObjectIterator.h"
#include "ISequencerModule.h"
#include "AudioControlBusTrackEditor.h"
#include "AudioControlBusMixTrackEditor.h"

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
#include "IMetaSoundGraphPanelPinFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundEditorModule.h"
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT

DEFINE_LOG_CATEGORY(LogAudioModulationEditor);

namespace AudioModulationEditor
{
	static const FName ToolName = TEXT("AssetTools");

	template <typename T>
	void AddAssetAction(IAssetTools& AssetTools, TArray<TSharedPtr<FAssetTypeActions_Base>>& AssetArray)
	{
		TSharedPtr<T> AssetAction = MakeShared<T>();
		TSharedPtr<FAssetTypeActions_Base> AssetActionBase = StaticCastSharedPtr<FAssetTypeActions_Base>(AssetAction);
		AssetTools.RegisterAssetTypeActions(AssetAction.ToSharedRef());
		AssetArray.Add(AssetActionBase);
	}
} // namespace AudioModulationEditor

FAudioModulationEditorModule::FAudioModulationEditorModule()
{
}

TSharedPtr<FExtensibilityManager> FAudioModulationEditorModule::GetModulationPatchMenuExtensibilityManager()
{
	return ModulationPatchMenuExtensibilityManager;
}

TSharedPtr<FExtensibilityManager> FAudioModulationEditorModule::GetModulationPatchToolbarExtensibilityManager()
{
	return ModulationPatchToolBarExtensibilityManager;
}

void FAudioModulationEditorModule::SetIcon(const FString& ClassName)
{
	static const FVector2D Icon16 = FVector2D(16.0f, 16.0f);
	static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

	static const FString IconDir = FPaths::EnginePluginsDir() / FString(TEXT("Runtime/AudioModulation/Content"));

	const FString IconFileName16 = FString::Printf(TEXT("%s_16x.png"), *ClassName);
	const FString IconFileName64 = FString::Printf(TEXT("%s_64x.png"), *ClassName);

	StyleSet->Set(*FString::Printf(TEXT("ClassIcon.%s"), *ClassName), new FSlateImageBrush(IconDir / IconFileName16, Icon16));
	StyleSet->Set(*FString::Printf(TEXT("ClassThumbnail.%s"), *ClassName), new FSlateImageBrush(IconDir / IconFileName64, Icon64));
}

void FAudioModulationEditorModule::StartupModule()
{
	// Seems to be required to avoid missing class definition on start-up for some platforms when the asset manager loads
	FModuleManager::LoadModuleChecked<FAudioModulationModule>("AudioModulation");

	static const FName AudioModulationStyleName(TEXT("AudioModulationStyle"));
	StyleSet = MakeShared<FSlateStyleSet>(AudioModulationStyleName);

	ModulationPatchToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();
	ModulationPatchMenuExtensibilityManager = MakeShared<FExtensibilityManager>();

	SetIcon(TEXT("SoundControlBus"));
	SetIcon(TEXT("SoundControlBusMix"));
	SetIcon(TEXT("SoundModulationGenerator"));
	SetIcon(TEXT("SoundModulationPatch"));
	SetIcon(TEXT("SoundModulationParameter"));

	RegisterCustomPropertyLayouts();

	ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>("CurveEditor");
	FModPatchCurveEditorModel::ModPatchViewId = CurveEditorModule.RegisterView(FOnCreateCurveEditorView::CreateStatic(
		[](TWeakPtr<FCurveEditor> WeakCurveEditor) -> TSharedRef<SCurveEditorView>
		{
			return SNew(SModulationPatchEditorViewStacked, WeakCurveEditor);
		}
	));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

	using namespace UE::MovieScene;

#if WITH_EDITOR
	if (GIsEditor)
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		AudioControlBusTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FAudioControlBusTrackEditor::CreateTrackEditor));
		AudioControlBusMixTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FAudioControlBusMixTrackEditor::CreateTrackEditor));
	}
#endif

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
	{
		using namespace Metasound;
		using namespace Metasound::Editor;
		IMetasoundEditorModule& Module = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
		TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = Module.GetGraphPanelPinFactory();
		PinFactory->RegisterPin("Modulator");
		PinFactory->RegisterPin(CreateArrayTypeNameFromElementTypeName("Modulator"));
		PinFactory->RegisterPin("ModulationParameter");
		PinFactory->RegisterPin(CreateArrayTypeNameFromElementTypeName("ModulationParameter"));
	}
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT

	// All parameters are required to always be loaded in editor to enable them to be referenced via object
	// metadata and custom layouts, even if they are not referenced by runtime uobjects/systems directly
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	/*
	AssetRegistry.OnAssetAdded().AddLambda([](const FAssetData& InAssetData)
	{
		UClass* AssetDataClass = InAssetData.GetClass();
		if (AssetDataClass && AssetDataClass->IsChildOf<USoundModulationParameter>())
		{
			if (USoundModulationParameter* Parameter = CastChecked<USoundModulationParameter>(InAssetData.GetAsset()))
			{
				Parameter->AddToRoot();
			}
		}
	});
	*/
	AssetRegistry.OnInMemoryAssetDeleted().AddLambda([](UObject* ObjectDeleted)
	{
		if (USoundModulationParameter* Parameter = Cast<USoundModulationParameter>(ObjectDeleted))
		{
			Parameter->RemoveFromRoot();
		}
	});
}

void FAudioModulationEditorModule::RegisterCustomPropertyLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyModule.RegisterCustomPropertyTypeLayout("SoundModulationTransform",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(
			&FSoundModulationTransformLayoutCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("SoundModulationParameterSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(
			&FSoundModulationParameterSettingsLayoutCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("SoundControlModulationPatch",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(
			&FSoundControlModulationPatchLayoutCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("SoundControlBusMixStage",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(
			&FSoundControlBusMixStageLayoutCustomization::MakeInstance));
}

void FAudioModulationEditorModule::ShutdownModule()
{
	ModulationPatchToolBarExtensibilityManager.Reset();
	ModulationPatchMenuExtensibilityManager.Reset();

	if (FModuleManager::Get().IsModuleLoaded(AudioModulationEditor::ToolName))
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>(AudioModulationEditor::ToolName).Get();
		for (TSharedPtr<FAssetTypeActions_Base>& AssetAction : AssetActions)
		{
			AssetTools.UnregisterAssetTypeActions(AssetAction.ToSharedRef());
		}
	}
	AssetActions.Reset();

	if (ICurveEditorModule* CurveEditorModule = FModuleManager::GetModulePtr<ICurveEditorModule>("CurveEditor"))
	{
		CurveEditorModule->UnregisterView(FModPatchCurveEditorModel::ModPatchViewId);
	}

	FModPatchCurveEditorModel::ModPatchViewId = ECurveEditorViewID::Invalid;

	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());

#if WITH_AUDIOMODULATION_METASOUND_SUPPORT
	if (FModuleManager::Get().IsModuleLoaded(Metasound::Editor::IMetasoundEditorModule::ModuleName))
	{
		using namespace Metasound;
		using namespace Metasound::Editor;
		IMetasoundEditorModule& Module = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
		TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = Module.GetGraphPanelPinFactory();
		PinFactory->UnregisterPin("Modulator");
		PinFactory->UnregisterPin(CreateArrayTypeNameFromElementTypeName("Modulator"));
		PinFactory->UnregisterPin("ModulationParameter");
		PinFactory->UnregisterPin(CreateArrayTypeNameFromElementTypeName("ModulationParameter"));
	}
#endif // WITH_AUDIOMODULATION_METASOUND_SUPPORT

#if WITH_EDITOR
	if (GIsEditor)
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.UnRegisterTrackEditor(AudioControlBusTrackCreateEditorHandle);
		SequencerModule.UnRegisterTrackEditor(AudioControlBusMixTrackCreateEditorHandle);
	}
#endif
}

IMPLEMENT_MODULE(FAudioModulationEditorModule, AudioModulationEditor);
