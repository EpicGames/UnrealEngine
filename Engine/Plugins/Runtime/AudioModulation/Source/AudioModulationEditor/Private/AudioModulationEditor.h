// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AssetTypeActions_Base.h"
#include "CurveEditorTypes.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAudioModulationEditor, Log, All);

class FAudioModulationEditorModule : public IModuleInterface
{
public:
	FAudioModulationEditorModule();

	TSharedPtr<FExtensibilityManager> GetModulationPatchMenuExtensibilityManager();
	TSharedPtr<FExtensibilityManager> GetModulationPatchToolbarExtensibilityManager();

	virtual void StartupModule() override;

	void RegisterCustomPropertyLayouts();

	virtual void ShutdownModule() override;

private:
	void SetIcon(const FString& ClassName);

	TArray<TSharedPtr<FAssetTypeActions_Base>> AssetActions;

	TSharedPtr<FExtensibilityManager> ModulationPatchMenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ModulationPatchToolBarExtensibilityManager;

	TSharedPtr<FSlateStyleSet> StyleSet;

	FDelegateHandle AudioControlBusTrackCreateEditorHandle;
	FDelegateHandle AudioControlBusMixTrackCreateEditorHandle;
};
