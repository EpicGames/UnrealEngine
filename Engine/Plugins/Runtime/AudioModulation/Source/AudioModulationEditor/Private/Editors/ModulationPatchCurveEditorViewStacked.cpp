// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModulationPatchCurveEditorViewStacked.h"

#include "Curves/CurveFloat.h"
#include "CurveEditor.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "IAudioModulation.h"
#include "Sound/SoundModulationDestination.h"
#include "SoundControlBus.h"
#include "SoundModulationParameter.h"
#include "SoundModulationPatch.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "ModulationPatchEditor"

namespace PatchCurveViewUtils
{
	static const FText NormalizedAxisName = LOCTEXT("ModulationCurveDisplayTitle_Normalized", "Normalized");

	void FormatLabel(const USoundModulationParameter& InParameter, const FNumberFormattingOptions& InNumFormatOptions, FText& InOutLabel)
	{
		const float NormalizedValue = FCString::Atof(*InOutLabel.ToString());
		const float UnitValue = InParameter.ConvertNormalizedToUnit(NormalizedValue);
		FText UnitLabel = FText::AsNumber(UnitValue, &InNumFormatOptions);

		// Label for the default parameter value
		if (InParameter.Settings.ValueNormalized == NormalizedValue)
		{
			InOutLabel = FText::Format(LOCTEXT("ModulationPatchCurveView_UnitFormat_Default", "{0} ({1}) (Default)"), UnitLabel, InOutLabel);
		}
		else
		{
			InOutLabel = FText::Format(LOCTEXT("ModulationPatchCurveView_UnitFormat", "{0} ({1})"), UnitLabel, InOutLabel);
		}
	}
} // namespace PatchCurveViewUtils

void SModulationPatchEditorViewStacked::FormatInputLabel(const WaveTable::Editor::FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const
{
	if (const USoundModulationPatch* Patch = static_cast<const FModPatchCurveEditorModel&>(EditorModel).GetPatch())
	{
		const int32 Index = EditorModel.GetCurveIndex();
		const FSoundControlModulationInput& Input = Patch->PatchSettings.Inputs[Index];
		if (const USoundControlBus* Bus = Input.Bus)
		{
			if (const USoundModulationParameter* Parameter = Bus->Parameter)
			{
				PatchCurveViewUtils::FormatLabel(*Parameter, InLabelFormat, InOutLabel);
			}
		}
	}
}

void SModulationPatchEditorViewStacked::FormatOutputLabel(const WaveTable::Editor::FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const
{
	if (const USoundModulationPatch* Patch = static_cast<const FModPatchCurveEditorModel&>(EditorModel).GetPatch())
	{
		if (const USoundModulationParameter* Parameter = Patch->PatchSettings.OutputParameter)
		{
			PatchCurveViewUtils::FormatLabel(*Parameter, InLabelFormat, InOutLabel);
		}
	}
}

FText SModulationPatchEditorViewStacked::FormatToolTipValue(const FCurveModel& CurveModel, double EvaluatedValue) const
{
	FNumberFormattingOptions FormatOptions;
	FormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;
	const FText NormalizedValue = FText::AsNumber(EvaluatedValue, &FormatOptions);
	const FModPatchCurveEditorModel& PatchCurveModel = static_cast<const FModPatchCurveEditorModel&>(CurveModel);
	const USoundModulationPatch* Patch = PatchCurveModel.GetPatch();
	FText UnitValue;
	FText UnitLabel;
	if (const TObjectPtr<USoundModulationParameter> Parameter = Patch->PatchSettings.OutputParameter)
	{
		const float Value = Parameter->ConvertNormalizedToUnit(EvaluatedValue);
		// Set the fractional digits based on the difference between the minimum and maximum value of the parameter.
		// Only add fractional digits so that the total number of digits is at least 4
		const float MinMaxDiff = Parameter->GetUnitMax() - Parameter->GetUnitMin();
		const int DiffDigits = FMath::Floor(FMath::LogX(10, FMath::Abs(MinMaxDiff)));
		FormatOptions.MaximumFractionalDigits = FMath::Clamp(MaximumFractionalDigits - DiffDigits, 0, MaximumFractionalDigits);
		UnitValue = FText::AsNumber(Value, &FormatOptions);
		UnitLabel = Parameter->Settings.UnitDisplayName;
	}
	return FText::Format(LOCTEXT("ModulationPatch_PointToolTipValue", "Value:  {0}{1} ({2})"), UnitValue, UnitLabel, NormalizedValue);
}

ECurveEditorViewID FModPatchCurveEditorModel::ModPatchViewId = ECurveEditorViewID::Invalid;

FModPatchCurveEditorModel::FModPatchCurveEditorModel(FRichCurve& InRichCurve, UObject* InOwner, EWaveTableCurveSource InSource)
	: WaveTable::Editor::FWaveTableCurveModel(InRichCurve, InOwner, InSource)
{
}

void FModPatchCurveEditorModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	if (ParentObject.IsValid())
	{
		if (USoundModulationPatch* Patch = CastChecked<USoundModulationPatch>(ParentObject))
		{
			if (TObjectPtr<USoundModulationParameter> Parameter = Patch->PatchSettings.OutputParameter)
			{
				if (!Parameter->IsModulationParameterClamped())
				{
					TRange<float> Range = Parameter->GetNormalizedLimits();
					MinValue = Range.GetLowerBoundValue();
					MaxValue = Range.GetUpperBoundValue();
					return;
				}
			}
		}
	}
	MinValue = 0.0f;
	MaxValue = 1.0f;
}

bool FModPatchCurveEditorModel::ShowDefaultDashedLine() const
{
	return true;
}

double FModPatchCurveEditorModel::GetDefaultDashedLineValue() const
{
	if (const USoundModulationPatch* Patch = GetPatch())
	{
		if (Patch->PatchSettings.OutputParameter)
		{
			return Patch->PatchSettings.OutputParameter->Settings.ValueNormalized;
		}
	}
	return 1.0;
}

bool FModPatchCurveEditorModel::GetIsBypassed() const
{
	if (const USoundModulationPatch* Patch = GetPatch())
	{
		return Patch->PatchSettings.bBypass;
	}

	return true;
}

USoundModulationPatch* FModPatchCurveEditorModel::GetPatch()
{
	if (ParentObject.IsValid())
	{
		return CastChecked<USoundModulationPatch>(ParentObject);
	}

	return nullptr;
}

const USoundModulationPatch* FModPatchCurveEditorModel::GetPatch() const
{
	if (ParentObject.IsValid())
	{
		return CastChecked<const USoundModulationPatch>(ParentObject);
	}

	return nullptr;
}

void FModPatchCurveEditorModel::RefreshCurveDescriptorText(const FWaveTableTransform& InTransform, FText& OutShortDisplayName, FText& OutInputAxisName, FText& OutOutputAxisName)
{
	OutShortDisplayName = LOCTEXT("ModulationCurveDisplayTitle_BusUnset", "Bus (Unset)");
	OutInputAxisName = PatchCurveViewUtils::NormalizedAxisName;
	OutOutputAxisName = PatchCurveViewUtils::NormalizedAxisName;

	if (const USoundModulationPatch* Patch = GetPatch())
	{
		const int32 Index = GetCurveIndex();

		const FSoundControlModulationInput& Input = Patch->PatchSettings.Inputs[Index];
		if (const USoundControlBus* Bus = Input.GetBus())
		{
			OutShortDisplayName = FText::FromString(Bus->GetName());

			if (const USoundModulationParameter* Parameter = Bus->Parameter)
			{
				static const FText AxisNameFormat = LOCTEXT("ModulationCurveDisplayTitle_InputAxisNameFormat", "{0} ({1})");
				OutInputAxisName = FText::Format(AxisNameFormat, FText::FromString(Parameter->GetName()), Parameter->Settings.UnitDisplayName);
			}
		}

		if (USoundModulationParameter* Parameter = Patch->PatchSettings.OutputParameter)
		{
			static const FText AxisNameFormat = LOCTEXT("ModulationCurveDisplayTitle_OutputAxisNameFormat", "{0} ({1})");
			OutOutputAxisName = FText::Format(AxisNameFormat, FText::FromString(Parameter->GetName()), Parameter->Settings.UnitDisplayName);
		}
	}
}
#undef LOCTEXT_NAMESPACE
