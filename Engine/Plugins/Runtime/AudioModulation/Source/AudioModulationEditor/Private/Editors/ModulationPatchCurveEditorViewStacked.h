// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioModulationStyle.h"
#include "Curves/RichCurve.h"
#include "RichCurveEditorModel.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WaveTableCurveEditorViewStacked.h"


// Forward Declarations
class UCurveFloat;
class USoundModulationParameter;
class USoundModulationPatch;
struct FSoundControlModulationInput;

#define LOCTEXT_NAMESPACE "AudioModulation"

class SModulationPatchEditorViewStacked : public WaveTable::Editor::SViewStacked
{
	virtual void FormatInputLabel(const WaveTable::Editor::FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const override;
	virtual void FormatOutputLabel(const WaveTable::Editor::FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const override;

protected:
	virtual FText FormatToolTipValue(const FCurveModel& CurveModel, double EvaluatedValue) const override;
	
	static constexpr int MaximumFractionalDigits = 3;
};

class FModPatchCurveEditorModel : public WaveTable::Editor::FWaveTableCurveModel
{
public:
	static ECurveEditorViewID ModPatchViewId;

	FModPatchCurveEditorModel(FRichCurve& InRichCurve, UObject* InOwner, EWaveTableCurveSource InSource);

	virtual void RefreshCurveDescriptorText(const FWaveTableTransform& InTransform, FText& OutShortDisplayName, FText& OutInputAxisName, FText& OutOutputAxisName) override;

	virtual ECurveEditorViewID GetViewId() const override { return ModPatchViewId; }
	virtual FColor GetCurveColor() const override { return UAudioModulationStyle::GetControlBusColor(); }
	virtual bool GetPropertyEditorDisabled() const override { return GetIsBypassed(); }
	virtual FText GetPropertyEditorDisabledText() const override { return LOCTEXT("ModulationPatchCurveEditorView_Bypassed", "Bypassed"); }

	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;

	virtual bool ShowDefaultDashedLine() const override;
	virtual double GetDefaultDashedLineValue() const override;
	
	bool GetIsBypassed() const;

	USoundModulationPatch* GetPatch();
	const USoundModulationPatch* GetPatch() const;
};
#undef LOCTEXT_NAMESPACE // "AudioModulation"
