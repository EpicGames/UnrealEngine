// Copyright Epic Games, Inc. All Rights Reserved.
#include "Editors/ModulationPatchEditor.h"

#include "AudioModulationEditor.h"
#include "AudioModulationStyle.h"
#include "ModulationPatchCurveEditorViewStacked.h"
#include "SoundModulationPatch.h"
#include "Templates/UniquePtr.h"
#include "WaveTableCurveEditorViewStacked.h"
#include "WaveTableSettings.h"
#include "WaveTableSampler.h"


#define LOCTEXT_NAMESPACE "ModulationPatchEditor"


TUniquePtr<WaveTable::Editor::FWaveTableCurveModel> FModulationPatchEditor::ConstructCurveModel(FRichCurve& InRichCurve, UObject* InParentObject, EWaveTableCurveSource InSource)
{
	using namespace WaveTable::Editor;
	return TUniquePtr<FWaveTableCurveModel>(new FModPatchCurveEditorModel(InRichCurve, GetEditingObject(), InSource));
}

bool FModulationPatchEditor::GetIsPropertyEditorDisabled() const
{
	return GetIsBypassed();
}

bool FModulationPatchEditor::GetIsBypassed() const
{
	if (USoundModulationPatch* Patch = Cast<USoundModulationPatch>(GetEditingObject()))
	{
		return Patch->PatchSettings.bBypass;
	}

	return false;
}

int32 FModulationPatchEditor::GetNumTransforms() const
{
	if (USoundModulationPatch* Patch = Cast<USoundModulationPatch>(GetEditingObject()))
	{
		return Patch->PatchSettings.Inputs.Num();
	}

	return 0;
}

FWaveTableTransform* FModulationPatchEditor::GetTransform(int32 InTransformIndex) const
{
	if (USoundModulationPatch* Patch = Cast<USoundModulationPatch>(GetEditingObject()))
	{
		FSoundControlModulationPatch& Settings = Patch->PatchSettings;
		if (Settings.Inputs.IsValidIndex(InTransformIndex))
		{
			return &Settings.Inputs[InTransformIndex].Transform;
		}
	}

	return nullptr;
}
#undef LOCTEXT_NAMESPACE
