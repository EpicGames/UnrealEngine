// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WaveTableBankEditor.h"


class FModulationPatchEditor : public WaveTable::Editor::FBankEditorBase
{
public:
	FModulationPatchEditor() = default;
	virtual ~FModulationPatchEditor() = default;

protected:
	virtual bool GetIsBypassed() const;
	virtual bool GetIsPropertyEditorDisabled() const override;
	virtual FWaveTableTransform* GetTransform(int32 InIndex) const override;
	virtual int32 GetNumTransforms() const override;

	virtual TUniquePtr<WaveTable::Editor::FWaveTableCurveModel> ConstructCurveModel(FRichCurve& InRichCurve, UObject* InParentObject, EWaveTableCurveSource InSource) override;
};
