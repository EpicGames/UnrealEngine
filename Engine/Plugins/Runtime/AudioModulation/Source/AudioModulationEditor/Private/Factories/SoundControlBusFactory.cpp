// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundControlBusFactory.h"

#include "AudioAnalytics.h"
#include "AudioModulationSettings.h"
#include "SoundControlBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundControlBusFactory)


USoundControlBusFactory::USoundControlBusFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundControlBus::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundControlBusFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage(TEXT("AudioModulation.ControlBusCreated"));
	USoundControlBus* NewControlBus = NewObject<USoundControlBus>(InParent, Name, Flags);
	if (NewControlBus)
	{
		if (const UAudioModulationSettings* ModulationSettings = GetDefault<UAudioModulationSettings>())
		{
			NewControlBus->Parameter = ModulationSettings->GetModulationParameter("Volume");
		}
	}
	return NewControlBus;
}
