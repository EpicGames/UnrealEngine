// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// USoundModulationBusFactory
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SoundControlBusFactory.generated.h"


UCLASS(hidecategories = Object, MinimalAPI)
class USoundControlBusFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};