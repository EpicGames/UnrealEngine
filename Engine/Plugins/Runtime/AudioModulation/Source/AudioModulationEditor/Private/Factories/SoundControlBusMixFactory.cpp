// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundControlBusMixFactory.h"

#include "AudioAnalytics.h"
#include "SoundControlBusMix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundControlBusMixFactory)


USoundControlBusMixFactory::USoundControlBusMixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundControlBusMix::StaticClass();
	bCreateNew     = true;
	bEditorImport  = false;
	bEditAfterNew  = true;
}

UObject* USoundControlBusMixFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage(TEXT("AudioModulation.ControlBusMixCreated"));
	return NewObject<USoundControlBusMix>(InParent, Name, Flags);
}

