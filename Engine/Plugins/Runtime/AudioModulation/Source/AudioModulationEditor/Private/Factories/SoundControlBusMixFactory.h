// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "SoundControlBusMixFactory.generated.h"


UCLASS(hidecategories=Object, MinimalAPI)
class USoundControlBusMixFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
