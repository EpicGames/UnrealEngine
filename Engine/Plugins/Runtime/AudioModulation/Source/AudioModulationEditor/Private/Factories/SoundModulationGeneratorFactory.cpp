// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModulationGeneratorFactory.h"

#include "AudioAnalytics.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Kismet2/SClassPickerDialog.h"
#include "SoundModulationGenerator.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationGeneratorFactory)

#define LOCTEXT_NAMESPACE "AudioModulation"


namespace AudioModulation
{
	namespace Editor
	{
		class FGeneratorClassViewerFilter : public IClassViewerFilter
		{
		public:
			TSet<const UClass*> AllowedChildrenOfClasses;
			EClassFlags DisallowedClassFlags;

			FGeneratorClassViewerFilter()
				: DisallowedClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists)
			{
				AllowedChildrenOfClasses.Add(USoundModulationGenerator::StaticClass());
			}

			virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
			{
				return !InClass->HasAnyClassFlags(DisallowedClassFlags)
					&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
			}

			virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
			{
				return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
					&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
			}
		};
	} // namespace Editor
} // namespace AudioModulation

USoundModulationGeneratorFactory::USoundModulationGeneratorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundModulationGenerator::StaticClass();
	bCreateNew     = true;
	bEditorImport  = false;
	bEditAfterNew  = true;
	GeneratorClass = nullptr;
}

bool USoundModulationGeneratorFactory::ConfigureProperties()
{
	using namespace AudioModulation::Editor;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.ClassFilters.Add(MakeShared<FGeneratorClassViewerFilter>());
	Options.Mode = EClassViewerMode::ClassPicker;

	const FText TitleText = LOCTEXT("CreateModulationGeneratorClassSelect", "Select Generator Class");

	UClass* PickedClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, PickedClass, USoundModulationGenerator::StaticClass());
	GeneratorClass = bPressedOk ? PickedClass : nullptr;

	return bPressedOk;
}

UObject* USoundModulationGeneratorFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (GeneratorClass)
	{
		Audio::Analytics::RecordEvent_Usage(TEXT("AudioModulation.ModulationGeneratorCreated"));
		return NewObject<USoundModulationGenerator>(InParent, GeneratorClass, InName, Flags);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE // AudioModulation

