// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"
#include "SoundModulationGenerator.h"

#include "SoundModulationGeneratorFactory.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class USoundModulationGeneratorFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TSubclassOf<USoundModulationGenerator> GeneratorClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
