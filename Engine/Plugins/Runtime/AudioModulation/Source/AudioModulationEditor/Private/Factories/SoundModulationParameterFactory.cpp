// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModulationParameterFactory.h"

#include "AudioAnalytics.h"
#include "ClassViewerFilter.h"
#include "SoundModulationParameter.h"
#include "Kismet2/SClassPickerDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationParameterFactory)


class FAssetClassParentFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

USoundModulationParameterFactory::USoundModulationParameterFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundModulationParameter::StaticClass();
	bCreateNew     = true;
	bEditorImport  = false;
	bEditAfterNew  = true;
	ParameterClass = nullptr;
}

bool USoundModulationParameterFactory::ConfigureProperties()
{
	ParameterClass = nullptr;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShared<FAssetClassParentFilter>();
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(USoundModulationParameter::StaticClass());

	const FText TitleText = NSLOCTEXT("AudioModulation", "CreateSoundModulationParameterOptions", "Select Parameter Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, USoundModulationParameter::StaticClass());

	if (bPressedOk)
	{
		ParameterClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* USoundModulationParameterFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundModulationParameter* NewParameter = nullptr;
	if (ParameterClass != nullptr)
	{
		NewParameter = NewObject<USoundModulationParameter>(InParent, ParameterClass, InName, Flags);

		Audio::Analytics::RecordEvent_Usage(TEXT("AudioModulation.ModulationParameterCreated"));
	}

	return NewParameter;
}
