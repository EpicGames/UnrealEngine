// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SoundModulationParameterFactory.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class USoundModulationParameterFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

private:
	UClass* ParameterClass;
};
