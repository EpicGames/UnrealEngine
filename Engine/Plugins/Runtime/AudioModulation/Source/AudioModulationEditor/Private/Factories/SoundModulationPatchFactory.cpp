// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationPatchFactory.h"

#include "AudioAnalytics.h"
#include "SoundModulationPatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationPatchFactory)


USoundModulationPatchFactory::USoundModulationPatchFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundModulationPatch::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundModulationPatchFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage(TEXT("AudioModulation.ParameterPatchCreated"));
	USoundModulationPatch* NewPatch = NewObject<USoundModulationPatch>(InParent, Name, Flags);
	
	if (NewPatch)
	{
		NewPatch->PatchSettings.bBypass = false;
		
		FSoundControlModulationInput DefaultInput;

		DefaultInput.Transform.Curve = EWaveTableCurve::Custom;
		DefaultInput.Transform.CurveCustom.AddKey(0.0f, 0.0f);
		DefaultInput.Transform.CurveCustom.AddKey(1.0f, 1.0f);

		NewPatch->PatchSettings.Inputs.Add(DefaultInput);
	}
	
	return NewPatch;
}

