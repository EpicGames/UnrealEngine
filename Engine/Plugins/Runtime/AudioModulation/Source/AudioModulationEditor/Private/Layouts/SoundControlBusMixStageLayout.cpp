// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlBusMixStageLayout.h"

#include "Audio.h"
#include "AudioModulationStyle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "IAudioModulation.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "InputCoreTypes.h"
#include "Misc/Attribute.h"
#include "PropertyRestriction.h"
#include "SCurveEditor.h"
#include "ScopedTransaction.h"
#include "Sound/SoundModulationDestination.h"
#include "SoundControlBus.h"
#include "SoundControlBusMix.h"
#include "SoundModulationParameterSettingsLayout.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "AudioModulation"


namespace AudioModulationEditorUtils
{
	USoundModulationParameter* GetParameterFromBus(TSharedRef<IPropertyHandle> BusHandle)
	{
		UObject* Object = nullptr;
		if (BusHandle->GetValue(Object) == FPropertyAccess::Success)
		{
			if (USoundControlBus* ControlBus = Cast<USoundControlBus>(Object))
			{
				return ControlBus->Parameter;
			}
		}

		return nullptr;
	}

	void HandleConvertNormalizedToUnit(TSharedRef<IPropertyHandle> BusHandle, TSharedRef<IPropertyHandle> NormalizedValueHandle, TSharedRef<IPropertyHandle> UnitValueHandle)
	{
		if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
		{
			if (Parameter->RequiresUnitConversion())
			{
				float NormalizedValue = 1.0f;
				if (NormalizedValueHandle->GetValue(NormalizedValue) == FPropertyAccess::Success)
				{
					const float UnitValue = Parameter->ConvertNormalizedToUnit(NormalizedValue);
					UnitValueHandle->SetValue(UnitValue, EPropertyValueSetFlags::NotTransactable);
				}
			}
			else
			{
				float NormalizedValue = 1.0f;
				NormalizedValueHandle->GetValue(NormalizedValue);
				UnitValueHandle->SetValue(NormalizedValue);
			}
		}
	}

	void HandleConvertUnitToNormalized(TSharedRef<IPropertyHandle> BusHandle, TSharedRef<IPropertyHandle> UnitValueHandle, TSharedRef<IPropertyHandle> NormalizedValueHandle)
	{
		if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
		{
			if (Parameter->RequiresUnitConversion())
			{
				float UnitValue = 1.0f;
				if (UnitValueHandle->GetValue(UnitValue) == FPropertyAccess::Success)
				{
					const float NormalizedValue = Parameter->ConvertUnitToNormalized(UnitValue);
					NormalizedValueHandle->SetValue(NormalizedValue);
				}
			}
			else
			{
				float UnitValue = 1.0f;
				UnitValueHandle->GetValue(UnitValue);
				NormalizedValueHandle->SetValue(UnitValue);
			}
		}
	}

	void HandleSetDefault(TSharedRef<IPropertyHandle> BusHandle, TSharedRef<IPropertyHandle> UnitValueHandle, TSharedRef<IPropertyHandle> NormalizedValueHandle)
	{
		if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
		{
			NormalizedValueHandle->SetValue(Parameter->Settings.ValueNormalized);
			UnitValueHandle->SetValue(Parameter->GetUnitDefault());
		}
		else
		{
			const Audio::FModulationParameter& DefaultParam = Audio::GetDefaultModulationParameter();

			float NormalizedValue = 1.0f;
			DefaultParam.GetDefaultNormalizedConversionFunction()(NormalizedValue);
			NormalizedValueHandle->SetValue(NormalizedValue);
			UnitValueHandle->SetValue(DefaultParam.DefaultValue);
		}
	}

	bool HandleIsDefaultValue(TSharedRef<IPropertyHandle> BusHandle, TSharedRef<IPropertyHandle> NormalizedValueHandle)
	{
		float ValueNormalized = 1.0f;
		NormalizedValueHandle->GetValue(ValueNormalized);

		if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
		{
			return FMath::IsNearlyEqual(Parameter->Settings.ValueNormalized, ValueNormalized);
		}

		return FMath::IsNearlyEqual(ValueNormalized, 1.0f);
	}
}

void FSoundControlBusMixStageLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.4f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		.MinDesiredWidth(300.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.4f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda(
					[StructPropertyHandle]()
					{
						constexpr bool bRecurse = false;
						if (TSharedPtr<IPropertyHandle> BusProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundControlBusMixStage, Bus), bRecurse))
						{
							const UObject* BusObject = nullptr;
							if (BusProperty->GetValue(BusObject) == FPropertyAccess::Success)
							{
								return FText::AsCultureInvariant(GetNameSafe(BusObject));
							}
						}

						return FText::Format(LOCTEXT("BusMixStageHeader_Format", "Bus Stage {0}"), StructPropertyHandle->GetPropertyDisplayName());
					}
				)
				.ToolTipText_Lambda(
					[StructPropertyHandle]()
					{
						constexpr bool bRecurse = false;
						if (TSharedPtr<IPropertyHandle> BusProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundControlBusMixStage, Bus), bRecurse))
						{
							const UObject* BusObject = nullptr;
							if (BusProperty->GetValue(BusObject) == FPropertyAccess::Success)
							{
								if (BusObject != nullptr)
								{
									UPackage* Package = BusObject->GetPackage();
									return FText::AsCultureInvariant(GetPathNameSafe(Package ? (UObject*)Package : BusObject));
								}
								return FText::GetEmpty();
							}
						}

						return StructPropertyHandle->GetToolTipText();
					}
				)
			]
		];
}

void FSoundControlBusMixStageLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	TSharedRef<IPropertyHandle> BusHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundControlBusMixStage, Bus)).ToSharedRef();
	TSharedRef<IPropertyHandle> NormalizedValueHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetValue)).ToSharedRef();
	TSharedRef<IPropertyHandle> UnitValueHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, TargetUnitValue)).ToSharedRef();

	auto ResetLambda = [BusHandle, NormalizedValueHandle, UnitValueHandle]()
	{
		AudioModulationEditorUtils::HandleSetDefault(BusHandle, UnitValueHandle, NormalizedValueHandle);
	};

	BusHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(ResetLambda));
	BusHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda(ResetLambda));
	ChildBuilder.AddProperty(BusHandle);

	// When editor opens, set unit value in case bus unit has changed while editor was closed.
	AudioModulationEditorUtils::HandleConvertNormalizedToUnit(BusHandle, NormalizedValueHandle, UnitValueHandle);

	NormalizedValueHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([BusHandle, NormalizedValueHandle, UnitValueHandle]()
	{
		AudioModulationEditorUtils::HandleConvertNormalizedToUnit(BusHandle, NormalizedValueHandle, UnitValueHandle);
	}));

	UnitValueHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([BusHandle, UnitValueHandle, NormalizedValueHandle]()
	{
		AudioModulationEditorUtils::HandleConvertUnitToNormalized(BusHandle, UnitValueHandle, NormalizedValueHandle);
		AudioModulationEditorUtils::HandleConvertNormalizedToUnit(BusHandle, NormalizedValueHandle, UnitValueHandle);
	}));

	FResetToDefaultOverride ResetToDefault = FResetToDefaultOverride::Create(
		TAttribute<bool>::Create([BusHandle, NormalizedValueHandle]()
		{
			return !AudioModulationEditorUtils::HandleIsDefaultValue(BusHandle, NormalizedValueHandle);
		}),
		FSimpleDelegate::CreateLambda(ResetLambda)
	);

	TSharedRef<SWidget> UnitValueWidget = UnitValueHandle->CreatePropertyValueWidget();
	TSharedRef<SWidget> NormalizedValueWidget = NormalizedValueHandle->CreatePropertyValueWidget();
	NormalizedValueWidget->SetVisibility(TAttribute<EVisibility>::Create([BusHandle]()
	{
		if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
		{
			return Parameter->RequiresUnitConversion() ? EVisibility::Visible : EVisibility::Hidden;
		}
	
		return EVisibility::Hidden;
	}));

	ChildBuilder.AddCustomRow(StructPropertyHandle->GetPropertyDisplayName())
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("BusModulationValue_MixValueName", "Value"))
		.ToolTipText(LOCTEXT("BusModulationValue_MixValueTooltip", "Value the mix stage is set to. If the bus's parameter has a unit conversion (eg. dB), there will be a unit value followed by a normalized (0-1) value. Changing one will update the other to have the corresponding value."))
	]
	.ValueContent()
		.MinDesiredWidth(300.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(0.455f)
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					UnitValueWidget
				]
			+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(TAttribute<FText>::Create([BusHandle]()
					{
						USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle);
						if (Parameter)
						{
							return Parameter->Settings.UnitDisplayName;
						}

						return FText();
					}))
					.Visibility(TAttribute<EVisibility>::Create([BusHandle]()
					{
						if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
						{
							return Parameter->Settings.UnitDisplayName.IsEmptyOrWhitespace() ? EVisibility::Collapsed : EVisibility::Visible;
						}

						return EVisibility::Visible;
					}))
				]
			+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					NormalizedValueWidget
				]
			+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SoundModulationControl_UnitValueNormalized", "Normalized"))
					.Visibility(TAttribute<EVisibility>::Create([BusHandle]()
					{
						if (USoundModulationParameter* Parameter = AudioModulationEditorUtils::GetParameterFromBus(BusHandle))
						{
							return Parameter->RequiresUnitConversion() ? EVisibility::Visible : EVisibility::Hidden;
						}

						return EVisibility::Hidden;
					}))
				]
		]
	.OverrideResetToDefault(ResetToDefault);

	TSharedRef<IPropertyHandle> AttackTimeHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, AttackTime)).ToSharedRef();
	ChildBuilder.AddProperty(AttackTimeHandle);

	TSharedRef<IPropertyHandle> ReleaseTimeHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoundModulationMixValue, ReleaseTime)).ToSharedRef();
	ChildBuilder.AddProperty(ReleaseTimeHandle);
}
#undef LOCTEXT_NAMESPACE // SoundModulationFloat
