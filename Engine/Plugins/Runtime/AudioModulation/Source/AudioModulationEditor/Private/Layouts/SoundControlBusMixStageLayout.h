// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "IAudioModulation.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SoundControlBus.h"


class FSoundControlBusMixStageLayoutCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundControlBusMixStageLayoutCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};
