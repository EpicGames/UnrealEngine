// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundControlModulationPatchLayout.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "IAudioModulation.h"
#include "SoundControlBus.h"
#include "SoundModulationParameterSettingsLayout.h"
#include "SoundModulationPatch.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "AudioModulation"
namespace AudioModulationEditorUtils
{
	void GetPropertyHandleMap(TSharedRef<IPropertyHandle> StructPropertyHandle, TMap<FName, TSharedPtr<IPropertyHandle>>& PropertyHandles)
	{
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}
	}

	TArray<USoundControlBus*> GetMismatchedBuses(TSharedRef<IPropertyHandle> InputsHandle, const USoundModulationParameter* InputParameter)
	{
		uint32 NumInputs;
		TSharedPtr<IPropertyHandleArray> InputArray = InputsHandle->AsArray();
		InputArray->GetNumElements(NumInputs);

		if (!InputParameter)
		{
			return TArray<USoundControlBus*>();
		}

		TArray<USoundControlBus*> MismatchBuses;
		for (uint32 i = 0; i < NumInputs; ++i)
		{
			TSharedRef<IPropertyHandle> Input = InputArray->GetElement(static_cast<int32>(i));
			TSharedRef<IPropertyHandle> BusInputHandle = Input->GetChildHandle("Bus", false /* bRecurse */).ToSharedRef();
			UObject* LinkedObj = nullptr;
			if (BusInputHandle->GetValue(LinkedObj) == FPropertyAccess::Success)
			{
				if (USoundControlBus* Bus = Cast<USoundControlBus>(LinkedObj))
				{
					if (Bus->Parameter)
					{
						if (Bus->Parameter != nullptr && Bus->Parameter != InputParameter)
						{
							MismatchBuses.Add(Bus);
						}
					}
				}
			}
		}

		return MismatchBuses;
	}
} // namespace AudioModulationEditorUtils

void FSoundControlModulationPatchLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
	AudioModulationEditorUtils::GetPropertyHandleMap(StructPropertyHandle, PropertyHandles);

	TSharedRef<IPropertyHandle> BypassHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundControlModulationPatch, bBypass)).ToSharedRef();
	TSharedRef<IPropertyHandle> InputsHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundControlModulationPatch, Inputs)).ToSharedRef();
	TSharedRef<IPropertyHandle> ParameterHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundControlModulationPatch, OutputParameter)).ToSharedRef();

	TAttribute<bool> EditCondition = TAttribute<bool>::Create([this, BypassHandle]()
	{
		bool bIsBypassed = false;
		BypassHandle->GetValue(bIsBypassed);
		return !bIsBypassed;
	});

	ChildBuilder.AddProperty(BypassHandle);
	ChildBuilder.AddProperty(ParameterHandle).EditCondition(EditCondition, nullptr);
	ChildBuilder.AddProperty(InputsHandle).EditCondition(EditCondition, nullptr);
}

void FSoundControlModulationPatchLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}
#undef LOCTEXT_NAMESPACE
