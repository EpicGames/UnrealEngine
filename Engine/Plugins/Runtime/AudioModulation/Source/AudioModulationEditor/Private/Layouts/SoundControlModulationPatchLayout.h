// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"


// Forward Declarations
class SSearchableComboBox;


class FSoundControlModulationPatchLayoutCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundControlModulationPatchLayoutCustomization>();
	}

private:
	template <typename T>
	void AddPatchProperties(TAttribute<EVisibility> VisibilityAttribute, TMap<FName, TSharedPtr<IPropertyHandle>>& PropertyHandles, IDetailChildrenBuilder& ChildBuilder)
	{
		TSharedPtr<IPropertyHandle>InputsHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(T, Inputs));
		ChildBuilder.AddProperty(InputsHandle.ToSharedRef())
			.Visibility(VisibilityAttribute);

		TSharedPtr<IPropertyHandle>OutputHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(T, Output));
		ChildBuilder.AddProperty(OutputHandle.ToSharedRef())
			.Visibility(VisibilityAttribute);
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

	TAttribute<EVisibility> CustomizeControl(TMap<FName, TSharedPtr<IPropertyHandle>>& PropertyHandles, IDetailChildrenBuilder& ChildBuilder);
};