// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationParameterSettingsLayout.h"

#include "Audio.h"
#include "AudioModulationStyle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "IAudioModulation.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "InputCoreTypes.h"
#include "Misc/Attribute.h"
#include "PropertyRestriction.h"
#include "SCurveEditor.h"
#include "ScopedTransaction.h"
#include "Sound/SoundModulationDestination.h"
#include "SoundControlBus.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "AudioModulation"


namespace ModParamSettingsUtils
{
	USoundModulationParameter* GetStructParameter(TSharedRef<IPropertyHandle> StructPropertyHandle)
	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);

		check(Objects.Num() == 1);
		return CastChecked<USoundModulationParameter>(Objects[0]);
	}
}

void FSoundModulationParameterSettingsLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FSoundModulationParameterSettingsLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	TSharedRef<IPropertyHandle> NormalizedValueHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationParameterSettings, ValueNormalized)).ToSharedRef();
	TSharedRef<IPropertyHandle> UnitValueHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationParameterSettings, ValueUnit)).ToSharedRef();
	TSharedRef<IPropertyHandle> UnitDisplayNameHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationParameterSettings, UnitDisplayName)).ToSharedRef();
	TSharedRef<IPropertyHandle> ClampToNormalizedHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationParameterSettings, bClampToNormalizedRange)).ToSharedRef();

	// Default Value Row (Unit)
	USoundModulationParameter* Parameter = ModParamSettingsUtils::GetStructParameter(StructPropertyHandle);
	ensure(Parameter);

	if (Parameter && Parameter->RequiresUnitConversion())
	{
		// When editor opens, set unit value in case bus unit has changed while editor was closed.
		Parameter->RefreshUnitValue();

		UnitValueHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([StructPropertyHandle]()
		{
			if (USoundModulationParameter* Parameter = ModParamSettingsUtils::GetStructParameter(StructPropertyHandle))
			{
				Parameter->RefreshNormalizedValue();
				Parameter->RefreshUnitValue();
			}
		}));

		NormalizedValueHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([StructPropertyHandle]()
		{
			if (USoundModulationParameter* Parameter = ModParamSettingsUtils::GetStructParameter(StructPropertyHandle))
			{
				Parameter->RefreshUnitValue();
			}
		}));

		ChildBuilder.AddCustomRow(StructPropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(FText::Format(LOCTEXT("BusModulationValue_DisplayNameFormat", "Default {0} Value "), StructPropertyHandle->GetPropertyDisplayName()))
			.ToolTipText(UnitValueHandle->GetToolTipText())
			.IsEnabled(false)
		]
		.ValueContent()
		.MinDesiredWidth(300.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					UnitValueHandle->CreatePropertyValueWidget()
				]
			+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(TAttribute<FText>::Create([StructPropertyHandle]()
					{
						if (USoundModulationParameter* Parameter = ModParamSettingsUtils::GetStructParameter(StructPropertyHandle))
						{
							if (Parameter->RequiresUnitConversion())
							{
								return Parameter->Settings.UnitDisplayName;
							}
						}
						return FText();
					}))
				]
			+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					NormalizedValueHandle->CreatePropertyValueWidget()
				]
			+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SoundModulationControl_UnitValueNormalized", "Normalized"))
					.IsEnabled(false)
				]
		];
	}
	// Default Value Row (No Unit)
	else
	{
		ChildBuilder.AddCustomRow(StructPropertyHandle->GetPropertyDisplayName())
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::Format(LOCTEXT("BusModulationValue_DisplayNameFormat", "Default {0} Value "), StructPropertyHandle->GetPropertyDisplayName()))
				.ToolTipText(StructPropertyHandle->GetToolTipText())
				.IsEnabled(false)
			]
			.ValueContent()
			[
				NormalizedValueHandle->CreatePropertyValueWidget()
			];
	}	

	TAttribute<bool> EditCondition = TAttribute<bool>::Create([Parameter]()
	{
		if (Parameter)
		{
			return Parameter->Settings.bShowClampOption;
		}
		return false;
	});
	
	ChildBuilder.AddProperty(UnitDisplayNameHandle);
	ChildBuilder.AddProperty(ClampToNormalizedHandle).EditCondition(EditCondition, nullptr);
}
#undef LOCTEXT_NAMESPACE // SoundModulationFloat
