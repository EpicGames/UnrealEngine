// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationTransformLayout.h"

#include "Containers/Array.h"
#include "SoundModulationPatch.h"
#include "UObject/NoExportTypes.h"


TSet<EWaveTableCurve> FSoundModulationTransformLayoutCustomization::GetSupportedCurves() const
{
	TSet<EWaveTableCurve> Curves = WaveTable::Editor::FTransformLayoutCustomizationBase::GetSupportedCurves();
	Curves.Remove(EWaveTableCurve::File);
	return Curves;
}

FWaveTableTransform* FSoundModulationTransformLayoutCustomization::GetTransform() const
{
	if (ensure(WaveTableOptionsHandle.IsValid()))
	{
		TArray<UObject*> OuterObjects;
		WaveTableOptionsHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Num() == 1)
		{
			if (USoundModulationPatch* Patch = Cast<USoundModulationPatch>(OuterObjects.Last()))
			{
				const int32 InputIndex = GetOwningArrayIndex();
				if (InputIndex != INDEX_NONE)
				{
					FSoundControlModulationPatch& PatchSettings = Patch->PatchSettings;
					if (InputIndex < PatchSettings.Inputs.Num())
					{
						return &PatchSettings.Inputs[InputIndex].Transform;
					}
				}
			}
		}
	}

	return nullptr;
}

bool FSoundModulationTransformLayoutCustomization::IsBipolar() const
{
	return false;
}
