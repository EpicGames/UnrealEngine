// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Curves/SimpleCurve.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SCurveEditor.h"
#include "WaveTableTransformLayout.h"


// Forward Declarations
struct FSoundControlModulationInput;
struct FSoundModulationTransform;
class USoundModulationPatch;

class FSoundModulationTransformLayoutCustomization : public WaveTable::Editor::FTransformLayoutCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundModulationTransformLayoutCustomization>();
	}

protected:
	//~ Begin FTransformLayoutCustomizationBase
	virtual TSet<EWaveTableCurve> GetSupportedCurves() const override;
	virtual FWaveTableTransform* GetTransform() const override;
	virtual bool IsBipolar() const override;
	//~ End FTransformLayoutCustomizationBase
};
