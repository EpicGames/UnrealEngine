// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "ClassTemplateEditorSubsystem.h"
#include "SoundModulationGenerator.h"

#include "AudioModulationClassTemplates.generated.h"


UCLASS(MinimalAPI, Abstract)
class USoundModulationClassTemplate : public UPluginClassTemplate
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class USoundModulationGeneratorClassTemplate : public USoundModulationClassTemplate
{
	GENERATED_UCLASS_BODY()
};
